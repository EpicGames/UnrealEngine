// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "Templates/SubclassOf.h"
#include "UObject/ScriptInterface.h"
#include "UObject/ScriptMacros.h"
#include "GameplayTaskOwnerInterface.h"
#include "UObject/WeakInterfacePtr.h"
#include "GameplayTaskTypes.h"
#include "GameplayTask.generated.h"

class AActor;
class UGameplayTaskResource;
class UGameplayTasksComponent;

GAMEPLAYTASKS_API DECLARE_LOG_CATEGORY_EXTERN(LogGameplayTasks, Log, All);

UENUM()
enum class EGameplayTaskState : uint8
{
	Uninitialized,
	AwaitingActivation,
	Paused,
	Active,
	Finished
};

UENUM()
enum class ETaskResourceOverlapPolicy : uint8
{
	/** Pause overlapping same-priority tasks. */
	StartOnTop,
	/** Wait for other same-priority tasks to finish. */
	StartAtEnd,
	/** Request to cancel same or lower priority tasks, if tasks not ended pause overlapping same-priority tasks. */
	RequestCancelAndStartOnTop,
	/** Request to cancel same or lower priority tasks, wait for any remaining overlapping same-priority tasks to finish. */
	RequestCancelAndStartAtEnd,
};
	
USTRUCT(BlueprintType)
struct FGameplayResourceSet
{
	GENERATED_USTRUCT_BODY()

	typedef uint16 FFlagContainer;
	typedef uint8 FResourceID;

	enum
	{
		MaxResources = sizeof(FFlagContainer)* 8
	};

private:
	FFlagContainer Flags;

public:
	/** Mind that this constructor takes _flags_ not individual IDs */
	explicit FGameplayResourceSet(FFlagContainer InFlags = 0) : Flags(InFlags)
	{}

	FFlagContainer GetFlags() const
	{
		return Flags;
	}
	bool IsEmpty() const
	{
		return Flags == 0;
	}
	FGameplayResourceSet& AddID(uint8 ResourceID)
	{
		ensure(ResourceID < MaxResources);
		Flags |= (1 << ResourceID);
		return *this;
	}
	FGameplayResourceSet& RemoveID(uint8 ResourceID)
	{
		ensure(ResourceID < MaxResources);
		Flags &= ~(1 << ResourceID);
		return *this;
	}
	bool HasID(uint8 ResourceID) const
	{
		ensure(ResourceID < MaxResources);
		return (Flags & (1 << ResourceID)) != 0;
	}
	FGameplayResourceSet& AddSet(FGameplayResourceSet Other)
	{
		Flags |= Other.Flags;
		return *this;
	}
	FGameplayResourceSet& RemoveSet(FGameplayResourceSet Other)
	{
		Flags &= ~Other.Flags;
		return *this;
	}
	void Clear()
	{
		Flags = FFlagContainer(0);
	}
	bool HasAllIDs(FGameplayResourceSet Other) const
	{
		return (Flags & Other.Flags) == Other.Flags;
	}
	bool HasAnyID(FGameplayResourceSet Other) const
	{
		return (Flags & Other.Flags) != 0;
	}
	FGameplayResourceSet GetOverlap(FGameplayResourceSet Other) const
	{
		return FGameplayResourceSet(Flags & Other.Flags);
	}
	FGameplayResourceSet GetDifference(FGameplayResourceSet Other) const
	{
		return FGameplayResourceSet(Flags & ~(Flags & Other.Flags));
	}

	bool operator==(const FGameplayResourceSet& Other) const
	{
		return Flags == Other.Flags;
	}

	bool operator!=(const FGameplayResourceSet& Other) const
	{
		return Flags != Other.Flags;
	}

	static FGameplayResourceSet AllResources()
	{
		return FGameplayResourceSet(FFlagContainer(-1));
	}

	static FGameplayResourceSet NoResources()
	{
		return FGameplayResourceSet(FFlagContainer(0));
	}

	GAMEPLAYTASKS_API FString GetDebugDescription() const;
};

UCLASS(Abstract, BlueprintType, meta = (ExposedAsyncProxy=AsyncTask), config = Game, MinimalAPI)
class UGameplayTask : public UObject, public IGameplayTaskOwnerInterface
{
	GENERATED_BODY()

public:

	DECLARE_DYNAMIC_MULTICAST_DELEGATE(FGenericGameplayTaskDelegate);

	GAMEPLAYTASKS_API UGameplayTask(const FObjectInitializer& ObjectInitializer);
	
	/** Called to trigger the actual task once the delegates have been set up */
	UFUNCTION(BlueprintCallable, meta = (BlueprintInternalUseOnly = "true"), Category = "Gameplay Tasks")
	GAMEPLAYTASKS_API void ReadyForActivation();

protected:
	/** Called to trigger the actual task once the delegates have been set up
	 *	Note that the default implementation does nothing and you don't have to call it */
	GAMEPLAYTASKS_API virtual void Activate();

	/** Initializes the task with the task owner interface instance but does not activate until Activate() is called */
	GAMEPLAYTASKS_API void InitTask(IGameplayTaskOwnerInterface& InTaskOwner, uint8 InPriority);

public:
	GAMEPLAYTASKS_API virtual void InitSimulatedTask(UGameplayTasksComponent& InGameplayTasksComponent);

	/** Tick function for this task, if bTickingTask == true */
	virtual void TickTask(float DeltaTime) {}

	/** Called when the task is asked to confirm from an outside node. What this means depends on the individual task. By default, this does nothing other than ending if bEndTask is true. */
	GAMEPLAYTASKS_API virtual void ExternalConfirm(bool bEndTask);

	/** Called when the task is asked to cancel from an outside node. What this means depends on the individual task. By default, this does nothing other than ending the task. */
	GAMEPLAYTASKS_API virtual void ExternalCancel();

	/** Called when a task owner wants to inform the task that it's done and don't want to get notified on task deactivation (i.e. OnGameplayTaskDeactivated). */
	void MarkOwnerFinished() { bOwnerFinished = true; }

	/** Return debug string describing task */
	GAMEPLAYTASKS_API virtual FString GetDebugString() const;

	/** Helper function for getting UWorld off a task */
	GAMEPLAYTASKS_API virtual UWorld* GetWorld() const override;

	/** Proper way to get the owning actor of task owner. This can be the owner itself since the owner is given as a interface */
	GAMEPLAYTASKS_API AActor* GetOwnerActor() const;

	/** Proper way to get the avatar actor associated with the task owner (usually a pawn, tower, etc) */
	GAMEPLAYTASKS_API AActor* GetAvatarActor() const;

	/** Helper function for instantiating and initializing a new task */
	template <class T>
	inline static T* NewTask(UObject* WorldContextObject, FName InstanceName = FName())
	{
		return WorldContextObject != nullptr ? NewTask<T>(*WorldContextObject, InstanceName) : nullptr;
	}

	template <class T>
	inline static T* NewTask(TScriptInterface<IGameplayTaskOwnerInterface> TaskOwner, FName InstanceName = FName())
	{
		return TaskOwner.GetInterface() != nullptr ? NewTask<T>(*TaskOwner, InstanceName) : nullptr;
	}

	template <class T>
	inline static T* NewTask(UObject& WorldContextObject, FName InstanceName = FName());

	template <class T>
	inline static T* NewTask(IGameplayTaskOwnerInterface& TaskOwner, FName InstanceName = FName());

	/** Added for consistency with NewTask, but to indicate that a task requires manual call to InitTask 
	 *	This path is used to manually configure some aspects of the task, like Priority */
	template <class T>
	inline static T* NewTaskUninitialized()
	{
		return NewObject<T>();
	}

	/** Called when task owner has "ended" (before the task ends) kills the task. Calls OnDestroy. */
	GAMEPLAYTASKS_API void TaskOwnerEnded();

	/** Called explicitly to end the task (usually by the task itself). Calls OnDestroy. 
	 *	@NOTE: you need to call EndTask before sending out any "on completed" delegates. 
	 *	If you don't the task will still be in an "active" state while the event receivers may
	 *	assume it's already "finished" */
	UFUNCTION(BlueprintCallable, Category="GameplayTasks")
	GAMEPLAYTASKS_API void EndTask();
	
	virtual bool IsSupportedForNetworking() const override { return bSimulatedTask; }

	inline FName GetInstanceName() const { return InstanceName; }
	inline bool IsTickingTask() const { return (bTickingTask != 0); }
	inline bool IsSimulatedTask() const { return (bSimulatedTask != 0); }
	inline bool IsSimulating() const { return (bIsSimulating != 0); }
	inline bool IsPausable() const { return (bIsPausable != 0); }
	inline bool HasOwnerFinished() const {	return (bOwnerFinished != 0); }
	inline uint8 GetPriority() const { return Priority; }
	inline bool RequiresPriorityOrResourceManagement() const { return bCaresAboutPriority == true || RequiredResources.IsEmpty() == false || ClaimedResources.IsEmpty() == false; }
	inline FGameplayResourceSet GetRequiredResources() const { return RequiredResources; }
	inline FGameplayResourceSet GetClaimedResources() const { return ClaimedResources; }
	
	inline EGameplayTaskState GetState() const { return TaskState; }
	inline bool IsActive() const { return (TaskState == EGameplayTaskState::Active); }
	inline bool IsPaused() const { return (TaskState == EGameplayTaskState::Paused); }
	inline bool IsFinished() const { return (TaskState == EGameplayTaskState::Finished); }
	
	UGameplayTask* GetChildTask() const { return ChildTask; }
	IGameplayTaskOwnerInterface* GetTaskOwner() const { return TaskOwner.IsValid() ? &(*TaskOwner) : nullptr; }
	UGameplayTasksComponent* GetGameplayTasksComponent() { return TasksComponent.Get(); }
	UGameplayTasksComponent* GetGameplayTasksComponent() const { return TasksComponent.Get(); }
	bool IsOwnedByTasksComponent() const { return bOwnedByTasksComponent; }

	template <class T>
	inline void AddRequiredResource()
	{
		AddRequiredResource(T::StaticClass());
	}

	template <class T>
	inline void AddClaimedResource()
	{
		AddClaimedResource(T::StaticClass());
	}

	/** Marks this task as requiring specified resource which has a number of consequences,
	 *	like task not being able to run if the resource is already taken.
	 *
	 *	@note: Calling this function makes sense only until the task is being passed over to the GameplayTasksComponent.
	 *	Once that's that resources data is consumed and further changes won't get applied 
	 */
	GAMEPLAYTASKS_API void AddRequiredResource(TSubclassOf<UGameplayTaskResource> RequiredResource);
	GAMEPLAYTASKS_API void AddRequiredResourceSet(const TArray<TSubclassOf<UGameplayTaskResource> >& RequiredResourceSet);
	GAMEPLAYTASKS_API void AddRequiredResourceSet(FGameplayResourceSet RequiredResourceSet);

	/** 
	 */
	GAMEPLAYTASKS_API void AddClaimedResource(TSubclassOf<UGameplayTaskResource> ClaimedResource);
	GAMEPLAYTASKS_API void AddClaimedResourceSet(const TArray<TSubclassOf<UGameplayTaskResource> >& AdditionalResourcesToClaim);
	GAMEPLAYTASKS_API void AddClaimedResourceSet(FGameplayResourceSet AdditionalResourcesToClaim);

	ETaskResourceOverlapPolicy GetResourceOverlapPolicy() const { return ResourceOverlapPolicy; }

	virtual bool IsWaitingOnRemotePlayerdata() const { return false; }

	virtual bool IsWaitingOnAvatar() const { return false; }

	GAMEPLAYTASKS_API FString GetTaskStateName() const;

protected:
	/** End and CleanUp the task - may be called by the task itself or by the task owner if the owner is ending. 
	 *	IMPORTANT! Do NOT call directly! Call EndTask() or TaskOwnerEnded() 
	 *	IMPORTANT! When overriding this function make sure to call Super::OnDestroy(bOwnerFinished) as the last thing,
	 *		since the function internally marks the task as "Pending Kill", and this may interfere with internal BP mechanics
	 */
	GAMEPLAYTASKS_API virtual void OnDestroy(bool bInOwnerFinished);

	static GAMEPLAYTASKS_API IGameplayTaskOwnerInterface* ConvertToTaskOwner(UObject& OwnerObject);
	static GAMEPLAYTASKS_API IGameplayTaskOwnerInterface* ConvertToTaskOwner(AActor& OwnerActor);

	// protected by design. Not meant to be called outside from GameplayTaskComponent mechanics
	GAMEPLAYTASKS_API virtual void Pause();
	GAMEPLAYTASKS_API virtual void Resume();

	// IGameplayTaskOwnerInterface BEGIN
	GAMEPLAYTASKS_API virtual UGameplayTasksComponent* GetGameplayTasksComponent(const UGameplayTask& Task) const override;
	GAMEPLAYTASKS_API virtual AActor* GetGameplayTaskOwner(const UGameplayTask* Task) const override;
	GAMEPLAYTASKS_API virtual AActor* GetGameplayTaskAvatar(const UGameplayTask* Task) const override;
	GAMEPLAYTASKS_API virtual uint8 GetGameplayTaskDefaultPriority() const override;
	GAMEPLAYTASKS_API virtual void OnGameplayTaskInitialized(UGameplayTask& Task) override;
	GAMEPLAYTASKS_API virtual void OnGameplayTaskDeactivated(UGameplayTask& Task) override;
	// IGameplayTaskOwnerInterface END

	/** Register all replication fragments */
	GAMEPLAYTASKS_API virtual void RegisterReplicationFragments(UE::Net::FFragmentRegistrationContext& Context, UE::Net::EFragmentRegistrationFlags RegistrationFlags) override;

private:
	friend UGameplayTasksComponent;

	GAMEPLAYTASKS_API void ActivateInTaskQueue();
	GAMEPLAYTASKS_API void PauseInTaskQueue();
	
	GAMEPLAYTASKS_API void PerformActivation();
	
protected:

	/** This name allows us to find the task later so that we can end it. */
	UPROPERTY()
	FName InstanceName;

	/** This controls how this task will be treated in relation to other, already running tasks, 
	 *	provided GameplayTasksComponent is configured to care about priorities (the default behavior)*/
	uint8 Priority;

	/** You should _never_ access it directly. We'll make it private once we have
	 *	a good way of "deprecating: direct access */
	EGameplayTaskState TaskState;

	UPROPERTY(config)
	ETaskResourceOverlapPolicy ResourceOverlapPolicy;

	/** If true, this task will receive TickTask calls from TasksComponent */
	uint32 bTickingTask : 1;

	/** Should this task run on simulated clients? This should only be used in rare cases, such as movement tasks. Simulated Tasks do not broadcast their end delegates.  */
	uint32 bSimulatedTask : 1;

	/** Am I actually running this as a simulated task. (This will be true on clients that simulating. This will be false on the server and the owning client) */
	uint32 bIsSimulating : 1;

	uint32 bIsPausable : 1;

	uint32 bCaresAboutPriority : 1;

	/** this is set to avoid duplicate calls to task's owner and TasksComponent when both are the same object */
	uint32 bOwnedByTasksComponent : 1;

	uint32 bClaimRequiredResources : 1;
	
	uint32 bOwnerFinished : 1;

	/** Abstract "resource" IDs this task needs available to be able to get activated. */
	FGameplayResourceSet RequiredResources;

	/**
	 *	Resources that are going to be locked when this task gets activated, but are not required to get this task started
	 */
	FGameplayResourceSet ClaimedResources;

	/** Task Owner that created us */
	TWeakInterfacePtr<IGameplayTaskOwnerInterface> TaskOwner;

	TWeakObjectPtr<UGameplayTasksComponent>	TasksComponent;

	/** child task instance */
	UPROPERTY()
	TObjectPtr<UGameplayTask> ChildTask;

#if WITH_GAMEPLAYTASK_DEBUG
	mutable FString DebugDescription;
public:
	const FString& GetDebugDescription() const
	{
		if (DebugDescription.IsEmpty())
		{
			DebugDescription = GenerateDebugDescription();
		}
		return DebugDescription;
	}
	GAMEPLAYTASKS_API virtual FString GenerateDebugDescription() const;
#endif // WITH_GAMEPLAYTASK_DEBUG
};

template <class T>
T* UGameplayTask::NewTask(UObject& WorldContextObject, FName InstanceName)
{
	IGameplayTaskOwnerInterface* TaskOwner = ConvertToTaskOwner(WorldContextObject);
	return (TaskOwner) ? NewTask<T>(*TaskOwner, InstanceName) : nullptr;
}

template <class T>
T* UGameplayTask::NewTask(IGameplayTaskOwnerInterface& TaskOwner, FName InstanceName)
{
	T* MyObj = NewObject<T>();
	MyObj->InstanceName = InstanceName;
	MyObj->InitTask(TaskOwner, TaskOwner.GetGameplayTaskDefaultPriority());
	return MyObj;
}

//For searching through lists of task instances
struct FGameplayTaskInstanceNamePredicate
{
	FGameplayTaskInstanceNamePredicate(FName DesiredInstanceName)
	{
		InstanceName = DesiredInstanceName;
	}

	bool operator()(const UGameplayTask* A) const
	{
		return (A && !A->GetInstanceName().IsNone() && A->GetInstanceName().IsValid() && (A->GetInstanceName() == InstanceName));
	}

	FName InstanceName;
};


struct FGameplayTaskInstanceClassPredicate
{
	FGameplayTaskInstanceClassPredicate(TSubclassOf<UGameplayTask> Class)
	{
		TaskClass = Class;
	}

	bool operator()(const UGameplayTask* A) const
	{
		return (A && (A->GetClass() == TaskClass));
	}

	TSubclassOf<UGameplayTask> TaskClass;
};

