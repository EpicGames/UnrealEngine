// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Interface.h"
#include "GameplayTaskTypes.h"
#include "GameplayTaskOwnerInterface.generated.h"

class AActor;
class UGameplayTask;
class UGameplayTasksComponent;

UINTERFACE(MinimalAPI, meta = (CannotImplementInterfaceInBlueprint))
class UGameplayTaskOwnerInterface : public UInterface
{
	GENERATED_BODY()
};

class IGameplayTaskOwnerInterface
{
	GENERATED_BODY()
public:

	/** Finds tasks component for given GameplayTask, Task.GetGameplayTasksComponent() may not be initialized at this point! */
	virtual UGameplayTasksComponent* GetGameplayTasksComponent(const UGameplayTask& Task) const PURE_VIRTUAL(IGameplayTaskOwnerInterface::GetGameplayTasksComponent, return nullptr;);

	/** Get owner of a task or default one when task is null */
	virtual AActor* GetGameplayTaskOwner(const UGameplayTask* Task) const PURE_VIRTUAL(IGameplayTaskOwnerInterface::GetGameplayTaskOwner, return nullptr;);

	/** Get "body" of task's owner / default, having location in world (e.g. Owner = AIController, Avatar = Pawn) */
	virtual AActor* GetGameplayTaskAvatar(const UGameplayTask* Task) const { return GetGameplayTaskOwner(Task); }

	/** Get default priority for running a task */
	virtual uint8 GetGameplayTaskDefaultPriority() const { return FGameplayTasks::DefaultPriority; }

	/** Notify called after GameplayTask finishes initialization (not active yet) */
	virtual void OnGameplayTaskInitialized(UGameplayTask& Task) {}
	
	/** Notify called after GameplayTask changes state to Active (initial activation or resuming) */
	virtual void OnGameplayTaskActivated(UGameplayTask& Task) {}

	/** Notify called after GameplayTask changes state from Active (finishing or pausing) */
	virtual void OnGameplayTaskDeactivated(UGameplayTask& Task) {}
};
