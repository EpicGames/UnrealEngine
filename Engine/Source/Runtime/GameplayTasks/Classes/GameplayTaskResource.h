// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "Templates/SubclassOf.h"
#include "GameplayTaskTypes.h"
#include "GameplayTaskResource.generated.h"

struct FPropertyChangedEvent;

UCLASS(Abstract, config = "Game", hidedropdown, MinimalAPI)
class UGameplayTaskResource : public UObject
{
	GENERATED_BODY()

protected:
	/** Overrides AutoResourceID. -1 means auto ID will be applied */
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = "Task Resource", meta = (ClampMin = "-1", ClampMax = "15", UIMin = "-1", UIMax = "15", EditCondition = "bManuallySetID"), config)
	int32 ManualResourceID;

private:
	UPROPERTY()
	int8 AutoResourceID;

public:
	UPROPERTY(EditDefaultsOnly, Category = "Task Resource", meta=(InlineEditConditionToggle))
	uint32 bManuallySetID : 1;

public:

	GAMEPLAYTASKS_API UGameplayTaskResource(const FObjectInitializer& ObjectInitializer);

	uint8 GetResourceID() const
	{
		return bManuallySetID || (ManualResourceID != INDEX_NONE) ? IntCastChecked<uint8>(ManualResourceID) : AutoResourceID;
	}

	template <class T>
	static uint8 GetResourceID()
	{
		return GetDefault<T>()->GetResourceID();
	}

	static uint8 GetResourceID(const TSubclassOf<UGameplayTaskResource>& RequiredResource)
	{
		return RequiredResource->GetDefaultObject<UGameplayTaskResource>()->GetResourceID();
	}

	GAMEPLAYTASKS_API virtual void PostInitProperties() override;

protected:
#if WITH_EDITOR
	GAMEPLAYTASKS_API void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent);
#endif // WITH_EDITOR

	GAMEPLAYTASKS_API void UpdateAutoResourceID();

#if WITH_GAMEPLAYTASK_DEBUG
protected:
	static GAMEPLAYTASKS_API TArray<FString> ResourceDescriptions;
	GAMEPLAYTASKS_API virtual FString GenerateDebugDescription() const;

public:
	static FString GetDebugDescription(uint8 ResourceId)
	{
		return ResourceDescriptions.IsValidIndex(ResourceId) ? ResourceDescriptions[ResourceId] : FString();
	}
#endif // WITH_GAMEPLAYTASK_DEBUG
};
