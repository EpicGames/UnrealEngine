// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Templates/SubclassOf.h"
#include "UObject/CoreNet.h"
#include "UObject/ScriptInterface.h"
#include "Components/ActorComponent.h"
#include "EngineDefines.h"
#include "GameplayTaskOwnerInterface.h"
#include "GameplayTask.h"
#include "GameplayTaskResource.h"
#include "VisualLogger/VisualLoggerDebugSnapshotInterface.h"
#include "GameplayTasksComponent.generated.h"

class AActor;
class Error;
class FOutBunch;
class UActorChannel;

enum class EGameplayTaskEvent : uint8
{
	Add,
	Remove,
};

UENUM()
enum class EGameplayTaskRunResult : uint8
{
	/** When tried running a null-task*/
	Error,
	Failed,
	/** Successfully registered for running, but currently paused due to higher priority tasks running */
	Success_Paused,
	/** Successfully activated */
	Success_Active,
	/** Successfully activated, but finished instantly */
	Success_Finished,
};

struct FGameplayTaskEventData
{
	EGameplayTaskEvent Event;
	UGameplayTask& RelatedTask;

	FGameplayTaskEventData(EGameplayTaskEvent InEvent, UGameplayTask& InRelatedTask)
		: Event(InEvent), RelatedTask(InRelatedTask)
	{

	}
};

DECLARE_DYNAMIC_MULTICAST_DELEGATE_TwoParams(FOnClaimedResourcesChangeSignature, FGameplayResourceSet, NewlyClaimed, FGameplayResourceSet, FreshlyReleased);

/**
*	The core ActorComponent for interfacing with the GameplayAbilities System
*/
UCLASS(ClassGroup = GameplayTasks, hidecategories = (Object, LOD, Lighting, Transform, Sockets, TextureStreaming), editinlinenew, meta = (BlueprintSpawnableComponent), MinimalAPI)
class UGameplayTasksComponent : public UActorComponent, public IGameplayTaskOwnerInterface, public IVisualLoggerDebugSnapshotInterface
{
	GENERATED_BODY()

private:

	friend struct FEventLock;
	int32 EventLockCounter;

	uint8 bInEventProcessingInProgress : 1;

protected:
	/** Indicates what's the highest priority among currently running tasks */
	uint8 TopActivePriority;

	/** Resources used by currently active tasks */
	FGameplayResourceSet CurrentlyClaimedResources;

	/** Tasks that run on simulated proxies */
	const TArray<UGameplayTask*>& GetSimulatedTasks()
	{
		return SimulatedTasks;
	}

	/** Remove all current tasks and register the one in the passed array. It's optimal to use Add/Remove of single tasks directly if possible.*/
	GAMEPLAYTASKS_API void SetSimulatedTasks(const TArray<UGameplayTask*>& NewSimulatedTasks);

	/** Add a new simulated task. Returns true if the task was added to the list. Returns false if the task was already registered. */
	GAMEPLAYTASKS_API bool AddSimulatedTask(UGameplayTask* NewTask);

	/** Remove an existing simulated task */
	GAMEPLAYTASKS_API void RemoveSimulatedTask(UGameplayTask* NewTask);

	UPROPERTY()
	TArray<TObjectPtr<UGameplayTask>> TaskPriorityQueue;
	
	/** Transient array of events whose main role is to avoid
	 *	long chain of recurrent calls if an activated/paused/removed task 
	 *	wants to push/pause/kill other tasks.
	 *	Note: this TaskEvents is assumed to be used in a single thread */
	TArray<FGameplayTaskEventData> TaskEvents;

	/** Array of currently active UGameplayTask that require ticking */
	UPROPERTY()
	TArray<TObjectPtr<UGameplayTask>> TickingTasks;

	/** All known tasks (processed by this component) referenced for GC */
	UPROPERTY(transient)
	TArray<TObjectPtr<UGameplayTask>> KnownTasks;

public:
	UPROPERTY(BlueprintReadWrite, Category = "Gameplay Tasks")
	FOnClaimedResourcesChangeSignature OnClaimedResourcesChange;

	GAMEPLAYTASKS_API UGameplayTasksComponent(const FObjectInitializer& ObjectInitializer);
	
	UFUNCTION()
	GAMEPLAYTASKS_API void OnRep_SimulatedTasks();

	GAMEPLAYTASKS_API virtual void GetLifetimeReplicatedProps(TArray< FLifetimeProperty >& OutLifetimeProps) const override;
	GAMEPLAYTASKS_API virtual bool ReplicateSubobjects(UActorChannel *Channel, FOutBunch* Bunch, FReplicationFlags* RepFlags) override;

	GAMEPLAYTASKS_API virtual void TickComponent(float DeltaTime, enum ELevelTick TickType, FActorComponentTickFunction* ThisTickFunction) override;
	GAMEPLAYTASKS_API void UpdateShouldTick();

	/** retrieves information whether this component should be ticking taken current
	*	activity into consideration*/
	GAMEPLAYTASKS_API virtual bool GetShouldTick() const;
	
	/** processes the task and figures out if it should get triggered instantly or wait
	 *	based on task's RequiredResources, Priority and ResourceOverlapPolicy */
	GAMEPLAYTASKS_API void AddTaskReadyForActivation(UGameplayTask& NewTask);

	GAMEPLAYTASKS_API void RemoveResourceConsumingTask(UGameplayTask& Task);
	GAMEPLAYTASKS_API void EndAllResourceConsumingTasksOwnedBy(const IGameplayTaskOwnerInterface& TaskOwner);

	GAMEPLAYTASKS_API bool FindAllResourceConsumingTasksOwnedBy(const IGameplayTaskOwnerInterface& TaskOwner, TArray<UGameplayTask*>& FoundTasks) const;
	
	/** finds first resource-consuming task of given name */
	GAMEPLAYTASKS_API UGameplayTask* FindResourceConsumingTaskByName(const FName TaskInstanceName) const;

	GAMEPLAYTASKS_API bool HasActiveTasks(UClass* TaskClass) const;

	inline FGameplayResourceSet GetCurrentlyUsedResources() const { return CurrentlyClaimedResources; }

	// BEGIN IGameplayTaskOwnerInterface
	virtual UGameplayTasksComponent* GetGameplayTasksComponent(const UGameplayTask& Task) const { return const_cast<UGameplayTasksComponent*>(this); }
	virtual AActor* GetGameplayTaskOwner(const UGameplayTask* Task) const override { return GetOwner(); }
	virtual AActor* GetGameplayTaskAvatar(const UGameplayTask* Task) const override { return GetOwner(); }
	GAMEPLAYTASKS_API virtual void OnGameplayTaskActivated(UGameplayTask& Task) override;
	GAMEPLAYTASKS_API virtual void OnGameplayTaskDeactivated(UGameplayTask& Task) override;
	// END IGameplayTaskOwnerInterface

	// ActorComponent overrides
	GAMEPLAYTASKS_API virtual void ReadyForReplication() override;

	UFUNCTION(BlueprintCallable, DisplayName="Run Gameplay Task", meta=(ScriptName="RunGameplayTask"), Category = "Gameplay Tasks", meta = (AutoCreateRefTerm = "AdditionalRequiredResources, AdditionalClaimedResources", AdvancedDisplay = "AdditionalRequiredResources, AdditionalClaimedResources"))
	static GAMEPLAYTASKS_API EGameplayTaskRunResult K2_RunGameplayTask(TScriptInterface<IGameplayTaskOwnerInterface> TaskOwner, UGameplayTask* Task, uint8 Priority, TArray<TSubclassOf<UGameplayTaskResource> > AdditionalRequiredResources, TArray<TSubclassOf<UGameplayTaskResource> > AdditionalClaimedResources);

	static GAMEPLAYTASKS_API EGameplayTaskRunResult RunGameplayTask(IGameplayTaskOwnerInterface& TaskOwner, UGameplayTask& Task, uint8 Priority, FGameplayResourceSet AdditionalRequiredResources, FGameplayResourceSet AdditionalClaimedResources);
	
#if WITH_GAMEPLAYTASK_DEBUG
	GAMEPLAYTASKS_API FString GetTickingTasksDescription() const;
	GAMEPLAYTASKS_API FString GetKnownTasksDescription() const;
	GAMEPLAYTASKS_API FString GetTasksPriorityQueueDescription() const;
	static GAMEPLAYTASKS_API FString GetTaskStateName(EGameplayTaskState Value);
#endif // WITH_GAMEPLAYTASK_DEBUG
	using GameplayTaskContainerType = decltype(TickingTasks);
	GAMEPLAYTASKS_API GameplayTaskContainerType::TConstIterator GetTickingTaskIterator() const;
	GAMEPLAYTASKS_API GameplayTaskContainerType::TConstIterator GetKnownTaskIterator() const;
	GAMEPLAYTASKS_API GameplayTaskContainerType::TConstIterator GetPriorityQueueIterator() const;
	GAMEPLAYTASKS_API GameplayTaskContainerType::TConstIterator GetSimulatedTaskIterator() const;

#if ENABLE_VISUAL_LOG
	GAMEPLAYTASKS_API virtual void GrabDebugSnapshot(FVisualLogEntry* Snapshot) const override;
	GAMEPLAYTASKS_API void DescribeSelfToVisLog(struct FVisualLogEntry* Snapshot) const;
#endif // ENABLE_VISUAL_LOG

protected:
	struct FEventLock
	{
		FEventLock(UGameplayTasksComponent* InOwner);
		~FEventLock();

	protected:
		UGameplayTasksComponent* Owner;
	};

	GAMEPLAYTASKS_API void RequestTicking();
	GAMEPLAYTASKS_API void ProcessTaskEvents();
	GAMEPLAYTASKS_API void UpdateTaskActivations();

	GAMEPLAYTASKS_API void SetCurrentlyClaimedResources(FGameplayResourceSet NewClaimedSet);

private:
	/** called when a task gets ended with an external call, meaning not coming from UGameplayTasksComponent mechanics */
	GAMEPLAYTASKS_API void OnTaskEnded(UGameplayTask& Task);

	GAMEPLAYTASKS_API void AddTaskToPriorityQueue(UGameplayTask& NewTask);
	GAMEPLAYTASKS_API void RemoveTaskFromPriorityQueue(UGameplayTask& Task);

	inline bool CanProcessEvents() const { return !bInEventProcessingInProgress && (EventLockCounter == 0); }

	GAMEPLAYTASKS_API void SetSimulatedTasksNetDirty();

	/** Tasks that run on simulated proxies */
	UPROPERTY(Transient, ReplicatedUsing = OnRep_SimulatedTasks)
	TArray<TObjectPtr<UGameplayTask>> SimulatedTasks;

	/** Previously replicated tasks that run on simulated proxies */
	TArray<TWeakObjectPtr<UGameplayTask>> PreviousOnRepSimulatedTasks;

};

typedef UGameplayTasksComponent::GameplayTaskContainerType::TConstIterator FConstGameplayTaskIterator;
