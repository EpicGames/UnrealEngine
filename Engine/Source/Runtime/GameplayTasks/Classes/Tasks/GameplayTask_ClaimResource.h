// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"
#include "Templates/SubclassOf.h"
#include "UObject/ScriptInterface.h"
#include "GameplayTaskOwnerInterface.h"
#include "GameplayTask.h"
#include "GameplayTaskResource.h"
#include "GameplayTask_ClaimResource.generated.h"

UCLASS(BlueprintType, MinimalAPI)
class UGameplayTask_ClaimResource : public UGameplayTask
{
	GENERATED_BODY()
public:
	GAMEPLAYTASKS_API UGameplayTask_ClaimResource(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get());

	UFUNCTION(BlueprintCallable, Category = "AI|Tasks", meta = (AdvancedDisplay = "Priority, TaskInstanceName"))
	static GAMEPLAYTASKS_API UGameplayTask_ClaimResource* ClaimResource(TScriptInterface<IGameplayTaskOwnerInterface> InTaskOwner, TSubclassOf<UGameplayTaskResource> ResourceClass, const uint8 Priority = 192, const FName TaskInstanceName = NAME_None);

	UFUNCTION(BlueprintCallable, Category = "AI|Tasks", meta = (AdvancedDisplay = "Priority, TaskInstanceName"))
	static GAMEPLAYTASKS_API UGameplayTask_ClaimResource* ClaimResources(TScriptInterface<IGameplayTaskOwnerInterface> InTaskOwner, TArray<TSubclassOf<UGameplayTaskResource> > ResourceClasses, const uint8 Priority = 192, const FName TaskInstanceName = NAME_None);

	static GAMEPLAYTASKS_API UGameplayTask_ClaimResource* ClaimResource(IGameplayTaskOwnerInterface& InTaskOwner, const TSubclassOf<UGameplayTaskResource> ResourceClass, const uint8 Priority = FGameplayTasks::DefaultPriority, const FName TaskInstanceName = NAME_None);
	static GAMEPLAYTASKS_API UGameplayTask_ClaimResource* ClaimResources(IGameplayTaskOwnerInterface& InTaskOwner, const TArray<TSubclassOf<UGameplayTaskResource> >& ResourceClasses, const uint8 Priority = FGameplayTasks::DefaultPriority, const FName TaskInstanceName = NAME_None);
};
