// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Templates/SubclassOf.h"
#include "UObject/ScriptInterface.h"
#include "GameFramework/Actor.h"
#include "GameplayTaskOwnerInterface.h"
#include "GameplayTask.h"
#include "GameplayTask_SpawnActor.generated.h"

DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FGameplayTaskSpawnActorDelegate, AActor*, SpawnedActor);

/**
 *	Convenience task for spawning actors (optionally limiting the spawning to the network authority). If not the net authority, we will not spawn 
 *	and Success will not be called. The nice thing this adds is the ability to modify expose on spawn properties while also implicitly checking 
 *	network role before spawning.
 *
 *	Though this task doesn't do much - games can implement similar tasks that carry out game specific rules. For example a 'SpawnProjectile'
 *	task that limits the available classes to the games projectile class, and that does game specific stuff on spawn (for example, determining
 *	firing position from a weapon attachment).
 *	
 *	Long term we can also use this task as a sync point. If the executing client could wait execution until the server creates and replicates the 
 *	actor down to it. We could potentially also use this to do predictive actor spawning / reconciliation.
 */

UCLASS(MinimalAPI)
class UGameplayTask_SpawnActor : public UGameplayTask
{
	GENERATED_BODY()
public:
	UPROPERTY(BlueprintAssignable)
	FGameplayTaskSpawnActorDelegate	Success;

	/** Called when we can't spawn: on clients or potentially on server if they fail to spawn (rare) */
	UPROPERTY(BlueprintAssignable)
	FGameplayTaskSpawnActorDelegate	DidNotSpawn;
	
	/** Spawn new Actor on the network authority (server) */
	UFUNCTION(BlueprintCallable, Category = "GameplayTasks", meta = (DisplayName="Spawn Actor for Gameplay Task", AdvancedDisplay = "TaskOwner, bSpawnOnlyOnAuthority", DefaultToSelf = "TaskOwner", BlueprintInternalUseOnly = "TRUE"))
	static UGameplayTask_SpawnActor* SpawnActor(TScriptInterface<IGameplayTaskOwnerInterface> TaskOwner, FVector SpawnLocation, FRotator SpawnRotation, TSubclassOf<AActor> Class, bool bSpawnOnlyOnAuthority = false);

	UFUNCTION(BlueprintCallable, meta = (WorldContext="WorldContextObject", BlueprintInternalUseOnly = "true"), Category = "GameplayTasks")
	virtual bool BeginSpawningActor(UObject* WorldContextObject, AActor*& SpawnedActor);

	UFUNCTION(BlueprintCallable, meta = (WorldContext="WorldContextObject", BlueprintInternalUseOnly = "true"), Category = "GameplayTasks")
	virtual void FinishSpawningActor(UObject* WorldContextObject, AActor* SpawnedActor);

protected:
	FVector CachedSpawnLocation;
	FRotator CachedSpawnRotation;
	UPROPERTY()
	TSubclassOf<AActor> ClassToSpawn;
};
