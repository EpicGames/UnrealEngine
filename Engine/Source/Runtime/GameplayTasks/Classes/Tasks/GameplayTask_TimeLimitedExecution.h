// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ScriptInterface.h"
#include "GameplayTaskOwnerInterface.h"
#include "GameplayTask.h"
#include "GameplayTask_TimeLimitedExecution.generated.h"

/**
 * Adds time limit for running a child task
 * - child task needs to be created with UGameplayTask_TimeLimitedExecution passed as TaskOwner 
 * - activations are tied together and when either UGameplayTask_TimeLimitedExecution or child task is activated, other one starts as well
 * - OnFinished and OnTimeExpired are mutually exclusive
 */
UCLASS(MinimalAPI)
class UGameplayTask_TimeLimitedExecution : public UGameplayTask
{
	GENERATED_BODY()

	DECLARE_DYNAMIC_MULTICAST_DELEGATE(FTaskFinishDelegate);

public:
	UGameplayTask_TimeLimitedExecution(const FObjectInitializer& ObjectInitializer);

	/** called when child task finishes execution before time runs out */
	UPROPERTY(BlueprintAssignable)
	FTaskFinishDelegate OnFinished;

	/** called when time runs out */
	UPROPERTY(BlueprintAssignable)
	FTaskFinishDelegate OnTimeExpired;

	virtual void Activate() override;
	virtual void OnGameplayTaskActivated(UGameplayTask& Task) override;
	virtual void OnGameplayTaskDeactivated(UGameplayTask& Task) override;

	/** Return debug string describing task */
	virtual FString GetDebugString() const override;

	GAMEPLAYTASKS_API static UGameplayTask_TimeLimitedExecution* LimitExecutionTime(IGameplayTaskOwnerInterface& InTaskOwner, float Time, const uint8 Priority = FGameplayTasks::DefaultPriority, const FName InInstanceName = FName());

private:

	void OnTimer();
	
	double Time;
	double TimeStarted;

	uint32 bTimeExpired : 1;
	uint32 bChildTaskFinished : 1;
};
