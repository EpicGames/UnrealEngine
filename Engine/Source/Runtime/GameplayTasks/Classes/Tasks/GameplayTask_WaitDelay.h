// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ScriptInterface.h"
#include "GameplayTaskOwnerInterface.h"
#include "GameplayTask.h"
#include "GameplayTask_WaitDelay.generated.h"

UCLASS(MinimalAPI)
class UGameplayTask_WaitDelay : public UGameplayTask
{
	GENERATED_BODY()

	DECLARE_DYNAMIC_MULTICAST_DELEGATE(FTaskDelayDelegate);

public:
	UGameplayTask_WaitDelay(const FObjectInitializer& ObjectInitializer);

	UPROPERTY(BlueprintAssignable)
	FTaskDelayDelegate OnFinish;

	virtual void Activate() override;

	/** Return debug string describing task */
	virtual FString GetDebugString() const override;

	/** Wait specified time. This is functionally the same as a standard Delay node. */
	UFUNCTION(BlueprintCallable, Category = "GameplayTasks", meta = (AdvancedDisplay = "TaskOwner, Priority", DefaultToSelf = "TaskOwner", BlueprintInternalUseOnly = "TRUE"))
	static UGameplayTask_WaitDelay* TaskWaitDelay(TScriptInterface<IGameplayTaskOwnerInterface> TaskOwner, float Time, const uint8 Priority = 192);

	GAMEPLAYTASKS_API static UGameplayTask_WaitDelay* TaskWaitDelay(IGameplayTaskOwnerInterface& InTaskOwner, float Time, const uint8 Priority = FGameplayTasks::DefaultPriority);

private:

	void OnTimeFinish();
	
	double Time;
	double TimeStarted;
};
