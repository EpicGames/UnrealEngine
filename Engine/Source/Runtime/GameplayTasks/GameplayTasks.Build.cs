// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class GameplayTasks : ModuleRules
	{
        public GameplayTasks(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"CoreUObject",
					"Engine",
					"NetCore"
                    //"GameplayTags",
					// ... add other public dependencies that you statically link with here ...
				}
				);

			if (Target.bBuildEditor == true)
			{
				PrivateDependencyModuleNames.Add("EditorFramework");
				PrivateDependencyModuleNames.Add("UnrealEd");
				CircularlyReferencedDependentModules.Add("UnrealEd");
                //PrivateDependencyModuleNames.Add("GameplayTagsEditor");
			}

			SetupIrisSupport(Target);
		}
	}
}
