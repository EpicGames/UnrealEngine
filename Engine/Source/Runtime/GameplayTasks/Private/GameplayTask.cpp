// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameplayTask.h"
#include "UObject/Package.h"
#include "GameFramework/Actor.h"
#include "VisualLogger/VisualLogger.h"
#include "GameplayTaskResource.h"
#include "GameplayTasksComponent.h"

#include "Iris/ReplicationSystem/ReplicationFragmentUtil.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GameplayTask)

UGameplayTask::UGameplayTask(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bTickingTask = false;
	bSimulatedTask = false;
	bIsSimulating = false;
	bOwnedByTasksComponent = false;
	bClaimRequiredResources = true;
	bOwnerFinished = false;
	TaskState = EGameplayTaskState::Uninitialized;
	ResourceOverlapPolicy = ETaskResourceOverlapPolicy::StartOnTop;
	Priority = FGameplayTasks::DefaultPriority;

	SetFlags(RF_StrongRefOnFrame);
}

IGameplayTaskOwnerInterface* UGameplayTask::ConvertToTaskOwner(UObject& OwnerObject)
{
	IGameplayTaskOwnerInterface* OwnerInterface = Cast<IGameplayTaskOwnerInterface>(&OwnerObject);

	if (OwnerInterface == nullptr)
	{
		AActor* AsActor = Cast<AActor>(&OwnerObject);
		if (AsActor)
		{
			OwnerInterface = AsActor->FindComponentByClass<UGameplayTasksComponent>();
		}
	}
	return OwnerInterface;
}

IGameplayTaskOwnerInterface* UGameplayTask::ConvertToTaskOwner(AActor& OwnerActor)
{
	IGameplayTaskOwnerInterface* OwnerInterface = Cast<IGameplayTaskOwnerInterface>(&OwnerActor);

	if (OwnerInterface == nullptr)
	{
		OwnerInterface = OwnerActor.FindComponentByClass<UGameplayTasksComponent>();
	}
	return OwnerInterface;
}

void UGameplayTask::ReadyForActivation()
{
	if (UGameplayTasksComponent* TasksPtr = TasksComponent.Get())
	{
		if (RequiresPriorityOrResourceManagement() == false)
		{
			PerformActivation();
		}
		else
		{
			TasksPtr->AddTaskReadyForActivation(*this);
		}
	}
	else
	{
		EndTask();
	}
}

void UGameplayTask::InitTask(IGameplayTaskOwnerInterface& InTaskOwner, uint8 InPriority)
{
	Priority = InPriority;
	TaskOwner = &InTaskOwner;
	TaskState = EGameplayTaskState::AwaitingActivation;

	if (bClaimRequiredResources)
	{
		ClaimedResources.AddSet(RequiredResources);
	}

	// call owner.OnGameplayTaskInitialized before accessing owner.GetGameplayTasksComponent, this is required for child tasks
	InTaskOwner.OnGameplayTaskInitialized(*this);

	UGameplayTasksComponent* GTComponent = InTaskOwner.GetGameplayTasksComponent(*this);
	TasksComponent = GTComponent;
	bOwnedByTasksComponent = (TaskOwner.GetObject() == GTComponent);

	// make sure that task component knows about new task
	if (GTComponent && !bOwnedByTasksComponent)
	{
		GTComponent->OnGameplayTaskInitialized(*this);
	}
}

void UGameplayTask::InitSimulatedTask(UGameplayTasksComponent& InGameplayTasksComponent)
{
	TasksComponent = &InGameplayTasksComponent;
	bIsSimulating = true;
}

UWorld* UGameplayTask::GetWorld() const
{
	if (UGameplayTasksComponent* TasksPtr = TasksComponent.Get())
	{
		return TasksPtr->GetWorld();
	}

	return nullptr;
}

AActor* UGameplayTask::GetOwnerActor() const
{
	if (TaskOwner.IsValid())
	{
		return TaskOwner->GetGameplayTaskOwner(this);		
	}
	else if (UGameplayTasksComponent* TasksPtr = TasksComponent.Get())
	{
		return TasksPtr->GetGameplayTaskOwner(this);
	}

	return nullptr;
}

AActor* UGameplayTask::GetAvatarActor() const
{
	if (TaskOwner.IsValid())
	{
		return TaskOwner->GetGameplayTaskAvatar(this);
	}
	else if (UGameplayTasksComponent* TasksPtr = TasksComponent.Get())
	{
		return TasksPtr->GetGameplayTaskAvatar(this);
	}

	return nullptr;
}

void UGameplayTask::TaskOwnerEnded()
{
	UE_VLOG(GetGameplayTasksComponent(), LogGameplayTasks, Verbose
		, TEXT("%s TaskOwnerEnded called, current State: %s")
		, *GetName(), *GetTaskStateName());

	if (TaskState != EGameplayTaskState::Finished)
	{
		bOwnerFinished = true;
		if (IsValidChecked(this))
		{
			OnDestroy(true);
		}
		else
		{
			// mark as finished, just to be on the safe side 
			TaskState = EGameplayTaskState::Finished;
		}
	}
}

void UGameplayTask::EndTask()
{
	UE_VLOG(GetGameplayTasksComponent(), LogGameplayTasks, Verbose
		, TEXT("%s EndTask called, current State: %s")
		, *GetName(), *GetTaskStateName());

	if (TaskState != EGameplayTaskState::Finished)
	{
		if (IsValidChecked(this))
		{
			OnDestroy(false);
		}
		else
		{
			// mark as finished, just to be on the safe side 
			TaskState = EGameplayTaskState::Finished;
		}
	}
}

void UGameplayTask::ExternalConfirm(bool bEndTask)
{
	UE_VLOG(GetGameplayTasksComponent(), LogGameplayTasks, Verbose
		, TEXT("%s ExternalConfirm called, bEndTask = %s, State : %s")
		, *GetName(), bEndTask ? TEXT("TRUE") : TEXT("FALSE"), *GetTaskStateName());

	if (bEndTask)
	{
		EndTask();
	}
}

void UGameplayTask::ExternalCancel()
{
	UE_VLOG(GetGameplayTasksComponent(), LogGameplayTasks, Verbose
		, TEXT("%s ExternalCancel called, current State: %s")
		, *GetName(), *GetTaskStateName());

	EndTask();
}

void UGameplayTask::OnDestroy(bool bInOwnerFinished)
{
	if (ensureMsgf(IsValidChecked(this), TEXT("OnDestroy called on invalid gameplay task")))
	{
		const FString OwnerName = TaskOwner.IsValid() ? TaskOwner.GetObject()->GetName() : TEXT("Invalid GameplayTask Owner");
		ensureMsgf(TaskState != EGameplayTaskState::Finished, TEXT("%s OnDestroy called, current state: %i, owner name: %s"), *GetName(), EnumToUnderlyingType(TaskState), *OwnerName);
	}
	TaskState = EGameplayTaskState::Finished;

	if (UGameplayTasksComponent* TasksPtr = TasksComponent.Get())
	{
		TasksPtr->OnGameplayTaskDeactivated(*this);
	}

	MarkAsGarbage();
}

FString UGameplayTask::GetDebugString() const
{
	return FString::Printf(TEXT("%s (%s)"), *GetName(), *InstanceName.ToString());
}

void UGameplayTask::AddRequiredResource(TSubclassOf<UGameplayTaskResource> RequiredResource)
{
	check(RequiredResource);
	const uint8 ResourceID = UGameplayTaskResource::GetResourceID(RequiredResource);
	RequiredResources.AddID(ResourceID);	
}

void UGameplayTask::AddRequiredResourceSet(const TArray<TSubclassOf<UGameplayTaskResource> >& RequiredResourceSet)
{
	for (auto Resource : RequiredResourceSet)
	{
		if (Resource)
		{
			const uint8 ResourceID = UGameplayTaskResource::GetResourceID(Resource);
			RequiredResources.AddID(ResourceID);
		}
	}
}

void UGameplayTask::AddRequiredResourceSet(FGameplayResourceSet RequiredResourceSet)
{
	RequiredResources.AddSet(RequiredResourceSet);
}

void UGameplayTask::AddClaimedResource(TSubclassOf<UGameplayTaskResource> ClaimedResource)
{
	check(ClaimedResource);
	const uint8 ResourceID = UGameplayTaskResource::GetResourceID(ClaimedResource);
	ClaimedResources.AddID(ResourceID);
}

void UGameplayTask::AddClaimedResourceSet(const TArray<TSubclassOf<UGameplayTaskResource> >& AdditionalResourcesToClaim)
{
	for (auto ResourceClass : AdditionalResourcesToClaim)
	{
		if (ResourceClass)
		{
			ClaimedResources.AddID(UGameplayTaskResource::GetResourceID(ResourceClass));
		}
	}
}

void UGameplayTask::AddClaimedResourceSet(FGameplayResourceSet AdditionalResourcesToClaim)
{
	ClaimedResources.AddSet(AdditionalResourcesToClaim);
}

void UGameplayTask::PerformActivation()
{
	if (TaskState == EGameplayTaskState::Active)
	{
		UE_VLOG(GetGameplayTasksComponent(), LogGameplayTasks, Warning
			, TEXT("%s PerformActivation called while TaskState is already Active. Bailing out.")
			, *GetName());
		return;
	}

	TaskState = EGameplayTaskState::Active;

	Activate();

	// Activate call may result in the task actually "instantly" finishing.
	// If this happens we don't want to bother the TaskComponent
	// with information on this task
	if (IsFinished() == false)
	{
		TasksComponent->OnGameplayTaskActivated(*this);
	}
}

void UGameplayTask::Activate()
{
	UE_VLOG(GetGameplayTasksComponent(), LogGameplayTasks, Verbose
		, TEXT("%s Activate called, current State: %s")
		, *GetName(), *GetTaskStateName());
}

void UGameplayTask::Pause()
{
	UE_VLOG(GetGameplayTasksComponent(), LogGameplayTasks, Verbose
		, TEXT("%s Pause called, current State: %s")
		, *GetName(), *GetTaskStateName());

	TaskState = EGameplayTaskState::Paused;

	TasksComponent->OnGameplayTaskDeactivated(*this);
}

void UGameplayTask::Resume()
{
	UE_VLOG(GetGameplayTasksComponent(), LogGameplayTasks, Verbose
		, TEXT("%s Resume called, current State: %s")
		, *GetName(), *GetTaskStateName());

	TaskState = EGameplayTaskState::Active;

	UE_CLOGF(TasksComponent.IsValid() == false, LogGameplayTasks, Error
		, "%ls Resume called while TasksComponent is no longer valid", *GetName());

	if (TasksComponent.IsValid())
	{
		TasksComponent->OnGameplayTaskActivated(*this);
	}
}

//----------------------------------------------------------------------//
// GameplayTasksComponent-related functions
//----------------------------------------------------------------------//
void UGameplayTask::ActivateInTaskQueue()
{
	switch(TaskState)
	{
	case EGameplayTaskState::Uninitialized:
		UE_VLOG(GetGameplayTasksComponent(), LogGameplayTasks, Error
			, TEXT("UGameplayTask::ActivateInTaskQueue Task %s passed for activation withouth having InitTask called on it!")
			, *GetName());
		break;
	case EGameplayTaskState::AwaitingActivation:
		PerformActivation();
		break;
	case EGameplayTaskState::Paused:
		// resume
		Resume();
		break;
	case EGameplayTaskState::Active:
		// nothing to do here
		break;
	case EGameplayTaskState::Finished:
		// If a task has finished, and it's being revived let's just treat the same as AwaitingActivation
		PerformActivation();
		break;
	default:
		checkNoEntry(); // looks like unhandled value! Probably a new enum entry has been added
		break;
	}
}

void UGameplayTask::PauseInTaskQueue()
{
	switch (TaskState)
	{
	case EGameplayTaskState::Uninitialized:
		UE_VLOG(GetGameplayTasksComponent(), LogGameplayTasks, Error
			, TEXT("UGameplayTask::PauseInTaskQueue Task %s passed for pausing withouth having InitTask called on it!")
			, *GetName());
		break;
	case EGameplayTaskState::AwaitingActivation:
		// nothing to do here. Don't change the state to indicate this task has never been run before
		break;
	case EGameplayTaskState::Paused:
		// nothing to do here. Already paused
		break;
	case EGameplayTaskState::Active:
		// pause!
		Pause();
		break;
	case EGameplayTaskState::Finished:
		// nothing to do here. But sounds odd, so let's log this, just in case
		UE_VLOG(GetGameplayTasksComponent(), LogGameplayTasks, Log
			, TEXT("UGameplayTask::PauseInTaskQueue Task %s being pause while already marked as Finished")
			, *GetName());
		break;
	default:
		checkNoEntry(); // looks like unhandled value! Probably a new enum entry has been added
		break;
	}
}

FString UGameplayTask::GetTaskStateName() const
{
	static const UEnum* Enum = StaticEnum<EGameplayTaskState>();
	check(Enum);
	return Enum->GetNameStringByValue(int64(TaskState));
}

#if WITH_GAMEPLAYTASK_DEBUG
//----------------------------------------------------------------------//
// debug
//----------------------------------------------------------------------//
FString UGameplayTask::GenerateDebugDescription() const
{
	if (RequiresPriorityOrResourceManagement())
	{
		UObject* OwnerOb = Cast<UObject>(GetTaskOwner());
		return FString::Printf(TEXT("%s:%s Pri:%d Owner:%s Res:%s"),
			*GetName(), InstanceName != NAME_None ? *InstanceName.ToString() : TEXT("-"),
			(int32)Priority,
			*GetNameSafe(OwnerOb),
			*RequiredResources.GetDebugDescription());
	}

	return GetName();
}
#endif // WITH_GAMEPLAYTASK_DEBUG

//////////////////////////////////////////////////////////////////////////
// Child tasks

UGameplayTasksComponent* UGameplayTask::GetGameplayTasksComponent(const UGameplayTask& Task) const
{
	return ((&Task == ChildTask) || (&Task == this)) ? GetGameplayTasksComponent() : nullptr;
}

AActor* UGameplayTask::GetGameplayTaskOwner(const UGameplayTask* Task) const
{
	return ((Task == ChildTask) || (Task == this)) ? UGameplayTask::GetOwnerActor() : nullptr;
}

AActor* UGameplayTask::GetGameplayTaskAvatar(const UGameplayTask* Task) const
{
	return ((Task == ChildTask) || (Task == this)) ? UGameplayTask::GetAvatarActor() : nullptr;
}

uint8 UGameplayTask::GetGameplayTaskDefaultPriority() const
{
	return GetPriority();
}

void UGameplayTask::OnGameplayTaskDeactivated(UGameplayTask& Task)
{
	// cleanup after deactivation
	if (&Task == ChildTask)
	{
		UE_VLOG(GetGameplayTasksComponent(), LogGameplayTasks, Verbose, TEXT("%s> Child task deactivated: %s (state: %s)"), *GetName(), *Task.GetName(), *Task.GetTaskStateName());
		if (Task.IsFinished())
		{
			ChildTask = nullptr;
		}
	}
}

void UGameplayTask::OnGameplayTaskInitialized(UGameplayTask& Task)
{
	UE_VLOG(GetGameplayTasksComponent(), LogGameplayTasks, Verbose, TEXT("%s> Child task initialized: %s"), *GetName(), *Task.GetName());

	// only one child task is allowed
	if (ChildTask)
	{
		UE_VLOG(GetGameplayTasksComponent(), LogGameplayTasks, Verbose, TEXT(">> terminating previous child task: %s"), *ChildTask->GetName());
		ChildTask->EndTask();
	}

	ChildTask = &Task;
}

void UGameplayTask::RegisterReplicationFragments(UE::Net::FFragmentRegistrationContext& Context, UE::Net::EFragmentRegistrationFlags RegistrationFlags)
{
	// Build descriptors and allocate PropertyReplicationFragments for this object.
	UE::Net::FReplicationFragmentUtil::CreateAndRegisterFragmentsForObject(this, Context, RegistrationFlags);
}
