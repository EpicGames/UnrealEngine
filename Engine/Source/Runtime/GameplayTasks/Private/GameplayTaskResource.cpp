// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameplayTaskResource.h"
#include "GameplayTask.h"
#include "Modules/ModuleManager.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GameplayTaskResource)

#if WITH_GAMEPLAYTASK_DEBUG
TArray<FString> UGameplayTaskResource::ResourceDescriptions;
#endif // WITH_GAMEPLAYTASK_DEBUG

#if WITH_RELOAD
namespace
{
	TMap<FName, int8> ClassNameToIDMap;
}
#endif // WITH_RELOAD

UGameplayTaskResource::UGameplayTaskResource(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bManuallySetID = false;
	ManualResourceID = INDEX_NONE;
	AutoResourceID = INDEX_NONE;
}

void UGameplayTaskResource::PostInitProperties()
{
	Super::PostInitProperties();

	if (HasAnyFlags(RF_ClassDefaultObject) && (GetClass()->HasAnyClassFlags(CLASS_Abstract) == false))
	{
#if WITH_RELOAD
		if (IsReloadActive())
		{
			if ((bManuallySetID == false || ManualResourceID == INDEX_NONE))
			{
				int8* PreviousID = ClassNameToIDMap.Find(GetFName());
				// PreviousID == null can happen if its the HOTRELOAD_CDO_DUPLICATE
				if (PreviousID)
				{
					AutoResourceID = *PreviousID;
				}
			}
		}
		else
#endif // WITH_RELOAD
		{
			if (bManuallySetID == false || ManualResourceID == INDEX_NONE)
			{
				UpdateAutoResourceID();
			}

			const uint8 DebugId = GetResourceID();
#if WITH_GAMEPLAYTASK_DEBUG
			ResourceDescriptions.SetNum(FMath::Max(DebugId + 1, ResourceDescriptions.Num()));
			ResourceDescriptions[DebugId] = GenerateDebugDescription();
#endif // WITH_GAMEPLAYTASK_DEBUG

#if WITH_RELOAD
			ClassNameToIDMap.Add(GetFName(), DebugId);
#endif // WITH_RELOAD
		}
	}
}

void UGameplayTaskResource::UpdateAutoResourceID()
{
	static uint16 NextAutoResID = 0;

	if (AutoResourceID == INDEX_NONE)
	{
		AutoResourceID = static_cast<int8>(NextAutoResID++);
		if (AutoResourceID >= FGameplayResourceSet::MaxResources)
		{
			UE_LOGF(LogGameplayTasks, Error, "AutoResourceID out of bounds (probably too much GameplayTaskResource classes, consider manually assigning values if you can split all classes into non-overlapping sets");
		}
	}
}

#if WITH_EDITOR
void UGameplayTaskResource::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	static const FName NAME_ManuallySetID = GET_MEMBER_NAME_CHECKED(UGameplayTaskResource, bManuallySetID);

	Super::PostEditChangeProperty(PropertyChangedEvent);

	if (PropertyChangedEvent.Property != NULL)
	{
		FName PropName = PropertyChangedEvent.Property->GetFName();

		if (PropName == NAME_ManuallySetID)
		{
			if (!bManuallySetID)
			{
				ManualResourceID = INDEX_NONE;
				// if we don't have ManualResourceID make sure AutoResourceID is valid
				UpdateAutoResourceID();
			}
		}
	}
}
#endif // WITH_EDITOR

#if WITH_GAMEPLAYTASK_DEBUG
FString UGameplayTaskResource::GenerateDebugDescription() const
{
	const FString ClassName = GetClass()->GetName();
	int32 SeparatorIdx = INDEX_NONE;
	if (ClassName.FindChar(TEXT('_'), SeparatorIdx))
	{
		return ClassName.Mid(SeparatorIdx + 1);
	}

	return ClassName;

}
#endif // WITH_GAMEPLAYTASK_DEBUG

