// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameplayTasksComponent.h"
#include "UObject/Package.h"
#include "Net/UnrealNetwork.h"
#include "Engine/ActorChannel.h"
#include "GameFramework/Actor.h"
#include "VisualLogger/VisualLoggerTypes.h"
#include "VisualLogger/VisualLogger.h"
#include "GameplayTasksPrivate.h"
#include "Logging/MessageLog.h"
#include "Net/Core/PushModel/PushModel.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GameplayTasksComponent)

#define LOCTEXT_NAMESPACE "GameplayTasksComponent"

namespace
{
	FORCEINLINE const TCHAR* GetGameplayTaskEventName(EGameplayTaskEvent Event)
	{
		/*static const UEnum* GameplayTaskEventEnum = StaticEnum<EGameplayTaskEvent>();
		return GameplayTaskEventEnum->GetDisplayNameTextByValue(static_cast<int64>(Event)).ToString();*/

		return Event == EGameplayTaskEvent::Add ? TEXT("Add") : TEXT("Remove");
	}
}

UGameplayTasksComponent::FEventLock::FEventLock(UGameplayTasksComponent* InOwner) : Owner(InOwner)
{
	if (Owner)
	{
		Owner->EventLockCounter++;
	}
}

UGameplayTasksComponent::FEventLock::~FEventLock()
{
	if (Owner)
	{
		Owner->EventLockCounter--;

		if (Owner->TaskEvents.Num() && Owner->CanProcessEvents())
		{
			Owner->ProcessTaskEvents();
		}
	}
}

UGameplayTasksComponent::UGameplayTasksComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	PrimaryComponentTick.TickGroup = TG_DuringPhysics;
	PrimaryComponentTick.bStartWithTickEnabled = false;
	PrimaryComponentTick.bCanEverTick = true;

	SetIsReplicatedByDefault(true);
	bInEventProcessingInProgress = false;
	TopActivePriority = 0;
}

void UGameplayTasksComponent::ReadyForReplication()
{
	Super::ReadyForReplication();

	REDIRECT_TO_VLOG(GetOwner());
	
	if (IsUsingRegisteredSubObjectList())
	{
		for (UGameplayTask* SimulatedTask : SimulatedTasks)
		{
			if (SimulatedTask)
			{
				AddReplicatedSubObject(SimulatedTask, COND_SkipOwner);
			}
		}
	}	
}
	
void UGameplayTasksComponent::OnGameplayTaskActivated(UGameplayTask& Task)
{
	// process events after finishing all operations
	FEventLock ScopeEventLock(this);
	KnownTasks.Add(&Task);

	if (Task.IsTickingTask())
	{
		check(TickingTasks.Contains(&Task) == false);
		TickingTasks.Add(&Task);

		// If this is our first ticking task, set this component as active so it begins ticking
		if (TickingTasks.Num() == 1)
		{
			UpdateShouldTick();
		}
	}
	
	if (Task.IsSimulatedTask())
	{
		const bool bWasAdded = AddSimulatedTask(&Task);
		check(bWasAdded == true);
	}

	IGameplayTaskOwnerInterface* TaskOwner = Task.GetTaskOwner();
	if (!Task.IsOwnedByTasksComponent() && TaskOwner)
	{
		TaskOwner->OnGameplayTaskActivated(Task);
	}
}

void UGameplayTasksComponent::OnGameplayTaskDeactivated(UGameplayTask& Task)
{
	// process events after finishing all operations
	FEventLock ScopeEventLock(this);
	const bool bIsFinished = (Task.GetState() == EGameplayTaskState::Finished);

	if (Task.GetChildTask() && bIsFinished)
	{
		if (Task.HasOwnerFinished())
		{
			Task.GetChildTask()->TaskOwnerEnded();
		}
		else
		{
			Task.GetChildTask()->EndTask();
		}
	}

	if (Task.IsTickingTask())
	{
		// If we are removing our last ticking task, set this component as inactive so it stops ticking
		TickingTasks.RemoveSingleSwap(&Task);
	}

	if (bIsFinished)
	{
		// using RemoveSwap rather than RemoveSingleSwap since a Task can be added
		// to KnownTasks both when activating as well as unpausing
		// while removal happens only once. It's cheaper to handle it here.
		KnownTasks.RemoveSwap(&Task);
	}

	if (Task.IsSimulatedTask())
	{
		RemoveSimulatedTask(&Task);
	}

	// Resource-using task
	if (Task.RequiresPriorityOrResourceManagement() && bIsFinished)
	{
		OnTaskEnded(Task);
	}

	IGameplayTaskOwnerInterface* TaskOwner = Task.GetTaskOwner();
	if (!Task.IsOwnedByTasksComponent() && !Task.HasOwnerFinished() && TaskOwner)
	{
		TaskOwner->OnGameplayTaskDeactivated(Task);
	}

	UpdateShouldTick();
}

void UGameplayTasksComponent::OnTaskEnded(UGameplayTask& Task)
{
	ensure(Task.RequiresPriorityOrResourceManagement() == true);
	RemoveResourceConsumingTask(Task);
}

void UGameplayTasksComponent::GetLifetimeReplicatedProps(TArray< FLifetimeProperty >& OutLifetimeProps) const
{
	// Intentionally not calling super: We do not want to replicate bActive which controls ticking. We sometimes need to tick on client predictively.
	DISABLE_ALL_CLASS_REPLICATED_PROPERTIES_FAST(Super, EFieldIteratorFlags::IncludeSuper);

	const FDoRepLifetimeParams Params{ COND_SkipOwner, REPNOTIFY_Always, true };
	DOREPLIFETIME_WITH_PARAMS_FAST(UGameplayTasksComponent, SimulatedTasks, Params);
}

bool UGameplayTasksComponent::ReplicateSubobjects(UActorChannel* Channel, class FOutBunch *Bunch, FReplicationFlags *RepFlags)
{
#if SUBOBJECT_TRANSITION_VALIDATION
	// When true it means we are calling this function to find any leftover replicated subobjects in classes that transitioned to the new registry list.
	// This shared class needs to keep supporting the old ways until we fully deprecate the API, so by only returning false we prevent the ensures to trigger
	if (UActorChannel::CanIgnoreDeprecatedReplicateSubObjects())
	{
		return false;
	}
#endif

	bool WroteSomething = Super::ReplicateSubobjects(Channel, Bunch, RepFlags);
	
	if (!RepFlags->bNetOwner)
	{
		for (UGameplayTask* SimulatedTask : GetSimulatedTasks())
		{
			if (IsValid(SimulatedTask))
			{
				WroteSomething |= Channel->ReplicateSubobject(SimulatedTask, *Bunch, *RepFlags);
			}
		}
	}

	return WroteSomething;
}

void UGameplayTasksComponent::OnRep_SimulatedTasks()
{
	if (IsUsingRegisteredSubObjectList())
	{
		// Find if any tasks got removed
		for (TWeakObjectPtr<UGameplayTask> OldSimulatedTaskPtr : PreviousOnRepSimulatedTasks)
		{
			UGameplayTask* OldSimulatedTask = OldSimulatedTaskPtr.Get();
			if (OldSimulatedTask)
			{
				const bool bIsRemoved = SimulatedTasks.Find(OldSimulatedTask) == INDEX_NONE;
				if (bIsRemoved)
				{
					RemoveReplicatedSubObject(OldSimulatedTask);
				}
			}
		}
	}

	for (UGameplayTask* SimulatedTask : GetSimulatedTasks())
	{
		if (SimulatedTask)
		{
			// If the task needs to be ticked and isn't yet.
			if (SimulatedTask->IsTickingTask() && TickingTasks.Contains(SimulatedTask) == false)
			{
				SimulatedTask->InitSimulatedTask(*this);

				TickingTasks.Add(SimulatedTask);

				// If this is our first ticking task, set this component as active so it begins ticking
				if (TickingTasks.Num() == 1)
				{
					UpdateShouldTick();
				}
			}

			// See if it's a new task that needs to be registered
			if (IsUsingRegisteredSubObjectList())
			{
				const bool bIsNew = PreviousOnRepSimulatedTasks.Find(SimulatedTask) == INDEX_NONE;
				if (bIsNew)
				{
					AddReplicatedSubObject(SimulatedTask, COND_SkipOwner);
				}
			}
		}
	}
	
	PreviousOnRepSimulatedTasks.Empty();
	PreviousOnRepSimulatedTasks.Append(SimulatedTasks);
}

void UGameplayTasksComponent::TickComponent(float DeltaTime, enum ELevelTick TickType, FActorComponentTickFunction *ThisTickFunction)
{
	SCOPE_CYCLE_COUNTER(STAT_TickGameplayTasks);

	Super::TickComponent(DeltaTime, TickType, ThisTickFunction);

	// Because we have no control over what a task may do when it ticks, we must be careful.
	// Ticking a task may kill the task right here. It could also potentially kill another task
	// which was waiting on the original task to do something. Since when a tasks is killed, it removes
	// itself from the TickingTask list, we will make a copy of the tasks we want to service before ticking any

	int32 NumTickingTasks = TickingTasks.Num();
	int32 NumActuallyTicked = 0;
	switch (NumTickingTasks)
	{
	case 0:
		break;
	case 1:
		{
			UGameplayTask* TickingTask = TickingTasks[0];
			if (IsValid(TickingTask))
			{
				TickingTask->TickTask(DeltaTime);
				NumActuallyTicked++;
			}
		}
		break;
	default:
		{
			static TArray<UGameplayTask*> LocalTickingTasks;
			LocalTickingTasks.Reset();
			LocalTickingTasks.Append(TickingTasks);
			for (UGameplayTask* TickingTask : LocalTickingTasks)
			{
				if (IsValid(TickingTask))
				{
					TickingTask->TickTask(DeltaTime);
					NumActuallyTicked++;
				}
			}
		}
		break;
	};

	// Stop ticking if no more active tasks
	if (NumActuallyTicked == 0)
	{
		TickingTasks.SetNum(0, EAllowShrinking::No);
		UpdateShouldTick();
	}
}

bool UGameplayTasksComponent::GetShouldTick() const
{
	return TickingTasks.Num() > 0;
}

void UGameplayTasksComponent::RequestTicking()
{
	if (IsActive() == false)
	{
		SetActive(true);
	}
}

void UGameplayTasksComponent::UpdateShouldTick()
{
	const bool bShouldTick = GetShouldTick();	
	if (IsActive() != bShouldTick)
	{
		SetActive(bShouldTick);
	}
}

//----------------------------------------------------------------------//
// Priority and resources handling
//----------------------------------------------------------------------//
void UGameplayTasksComponent::AddTaskReadyForActivation(UGameplayTask& NewTask)
{
	UE_VLOG(this, LogGameplayTasks, Log, TEXT("AddTaskReadyForActivation %s"), *NewTask.GetName());

	ensure(NewTask.RequiresPriorityOrResourceManagement() == true);
	
	TaskEvents.Add(FGameplayTaskEventData(EGameplayTaskEvent::Add, NewTask));
	// trigger the actual processing only if it was the first event added to the list
	if (TaskEvents.Num() == 1 && CanProcessEvents())
	{
		ProcessTaskEvents();
	}
}

void UGameplayTasksComponent::RemoveResourceConsumingTask(UGameplayTask& Task)
{
	UE_VLOG(this, LogGameplayTasks, Log, TEXT("RemoveResourceConsumingTask %s"), *Task.GetName());

	TaskEvents.Add(FGameplayTaskEventData(EGameplayTaskEvent::Remove, Task));
	// trigger the actual processing only if it was the first event added to the list
	if (TaskEvents.Num() == 1 && CanProcessEvents())
	{
		ProcessTaskEvents();
	}
}

void UGameplayTasksComponent::EndAllResourceConsumingTasksOwnedBy(const IGameplayTaskOwnerInterface& TaskOwner)
{
	FEventLock ScopeEventLock(this);

	for (int32 Idx = 0; Idx < TaskPriorityQueue.Num(); Idx++)
	{
		if (TaskPriorityQueue[Idx] && TaskPriorityQueue[Idx]->GetTaskOwner() == &TaskOwner)
		{
			// finish task, remove event will be processed after all locks are cleared
			TaskPriorityQueue[Idx]->TaskOwnerEnded();
		}
	}
}

bool UGameplayTasksComponent::FindAllResourceConsumingTasksOwnedBy(const IGameplayTaskOwnerInterface& TaskOwner, TArray<UGameplayTask*>& FoundTasks) const
{
	int32 NumFound = 0;
	for (int32 TaskIndex = 0; TaskIndex < TaskPriorityQueue.Num(); TaskIndex++)
	{
		if (TaskPriorityQueue[TaskIndex] && TaskPriorityQueue[TaskIndex]->GetTaskOwner() == &TaskOwner)
		{
			FoundTasks.Add(TaskPriorityQueue[TaskIndex]);
			NumFound++;
		}
	}

	return (NumFound > 0);
}

UGameplayTask* UGameplayTasksComponent::FindResourceConsumingTaskByName(const FName TaskInstanceName) const
{
	for (int32 TaskIndex = 0; TaskIndex < TaskPriorityQueue.Num(); TaskIndex++)
	{
		if (TaskPriorityQueue[TaskIndex] && TaskPriorityQueue[TaskIndex]->GetInstanceName() == TaskInstanceName)
		{
			return TaskPriorityQueue[TaskIndex];
		}
	}

	return nullptr;
}

bool UGameplayTasksComponent::HasActiveTasks(UClass* TaskClass) const
{
	for (int32 Idx = 0; Idx < KnownTasks.Num(); Idx++)
	{
		if (KnownTasks[Idx] && KnownTasks[Idx]->IsA(TaskClass))
		{
			return true;
		}
	}

	return false;
}

void UGameplayTasksComponent::ProcessTaskEvents()
{
	static const int32 MaxIterations = 16;
	bInEventProcessingInProgress = true;

	int32 IterCounter = 0;
	while (TaskEvents.Num() > 0)
	{
		IterCounter++;
		if (IterCounter > MaxIterations)
		{
			UE_VLOG(this, LogGameplayTasks, Error, TEXT("UGameplayTasksComponent::ProcessTaskEvents has exceeded allowes number of iterations. Check your GameplayTasks for logic loops!"));
			TaskEvents.Reset();
			break;
		}

		for (int32 EventIndex = 0; EventIndex < TaskEvents.Num(); ++EventIndex)
		{
			UE_VLOG(this, LogGameplayTasks, Verbose, TEXT("UGameplayTasksComponent::ProcessTaskEvents: %s event %s")
				, *TaskEvents[EventIndex].RelatedTask.GetName(), GetGameplayTaskEventName(TaskEvents[EventIndex].Event));

			if (!IsValid(&TaskEvents[EventIndex].RelatedTask))
			{
				UE_VLOG(this, LogGameplayTasks, Verbose, TEXT("%s is invalid"), *TaskEvents[EventIndex].RelatedTask.GetName());
				// we should ignore it, but just in case run the removal code.
				RemoveTaskFromPriorityQueue(TaskEvents[EventIndex].RelatedTask);
				continue;
			}

			switch (TaskEvents[EventIndex].Event)
			{
			case EGameplayTaskEvent::Add:
				if (TaskEvents[EventIndex].RelatedTask.TaskState != EGameplayTaskState::Finished)
				{
					AddTaskToPriorityQueue(TaskEvents[EventIndex].RelatedTask);
				}
				else
				{
					UE_VLOG(this, LogGameplayTasks, Error, TEXT("UGameplayTasksComponent::ProcessTaskEvents trying to add a finished task to priority queue!"));
				}
				break;
			case EGameplayTaskEvent::Remove:
				RemoveTaskFromPriorityQueue(TaskEvents[EventIndex].RelatedTask);
				break;
			default:
				checkNoEntry();
				break;
			}
		}

		TaskEvents.Reset();
		UpdateTaskActivations();

		// task activation changes may create new events, loop over to check it
	}

	bInEventProcessingInProgress = false;
}

void UGameplayTasksComponent::AddTaskToPriorityQueue(UGameplayTask& NewTask)
{
	if ((NewTask.GetResourceOverlapPolicy() == ETaskResourceOverlapPolicy::RequestCancelAndStartOnTop)
		|| (NewTask.GetResourceOverlapPolicy() == ETaskResourceOverlapPolicy::RequestCancelAndStartAtEnd))
	{
		const FGameplayResourceSet NewClaimedResources = NewTask.GetClaimedResources();
		TArray<UGameplayTask*, TInlineAllocator<2>> CancelList;

		for (UGameplayTask* Task : TaskPriorityQueue)
		{
			if (Task != nullptr
				&& Task->GetPriority() <= NewTask.GetPriority()
				&& Task->GetClaimedResources().HasAnyID(NewClaimedResources))
			{
				// Postpone cancelling, as cancel can call EndTask() and may alter the TaskPriorityQueue.  
				CancelList.Add(Task);
			}
		}

		for (UGameplayTask* Task : CancelList)
		{
			Task->ExternalCancel();
		}
	}
	
	const bool bStartOnTopOfSamePriority = (NewTask.GetResourceOverlapPolicy() == ETaskResourceOverlapPolicy::StartOnTop)
										|| (NewTask.GetResourceOverlapPolicy() == ETaskResourceOverlapPolicy::RequestCancelAndStartOnTop);
	int32 InsertionPoint = INDEX_NONE;
	
	for (int32 Idx = 0; Idx < TaskPriorityQueue.Num(); ++Idx)
	{
		if (TaskPriorityQueue[Idx] == nullptr)
		{
			continue;
		}

		if ((bStartOnTopOfSamePriority && TaskPriorityQueue[Idx]->GetPriority() <= NewTask.GetPriority())
			|| (!bStartOnTopOfSamePriority && TaskPriorityQueue[Idx]->GetPriority() < NewTask.GetPriority()))
		{
			TaskPriorityQueue.Insert(&NewTask, Idx);
			InsertionPoint = Idx;
			break;
		}
	}
	
	if (InsertionPoint == INDEX_NONE)
	{
		TaskPriorityQueue.Add(&NewTask);
	}
}

void UGameplayTasksComponent::RemoveTaskFromPriorityQueue(UGameplayTask& Task)
{	
	const int32 RemovedTaskIndex = TaskPriorityQueue.Find(&Task);
	if (RemovedTaskIndex != INDEX_NONE)
	{
		TaskPriorityQueue.RemoveAt(RemovedTaskIndex, EAllowShrinking::No);
	}
	else
	{
		// take a note and ignore
		UE_VLOG(this, LogGameplayTasks, Verbose, TEXT("RemoveTaskFromPriorityQueue for %s called, but it's not in the queue. Might have been already removed"), *Task.GetName());
	}
}

void UGameplayTasksComponent::UpdateTaskActivations()
{
	FGameplayResourceSet ResourcesClaimed;
	bool bHasNulls = false;

	if (TaskPriorityQueue.Num() > 0)
	{
		TArray<UGameplayTask*> ActivationList;
		ActivationList.Reserve(TaskPriorityQueue.Num());

		FGameplayResourceSet ResourcesBlocked;
		for (int32 TaskIndex = 0; TaskIndex < TaskPriorityQueue.Num(); ++TaskIndex)
		{
			if (TaskPriorityQueue[TaskIndex])
			{
				const FGameplayResourceSet RequiredResources = TaskPriorityQueue[TaskIndex]->GetRequiredResources();
				const FGameplayResourceSet ClaimedResources = TaskPriorityQueue[TaskIndex]->GetClaimedResources();
				if (RequiredResources.GetOverlap(ResourcesBlocked).IsEmpty())
				{
					// postpone activations, it's some tasks (like MoveTo) require pausing old ones first
					ActivationList.Add(TaskPriorityQueue[TaskIndex]);
					ResourcesClaimed.AddSet(ClaimedResources);
				}
				else
				{
					TaskPriorityQueue[TaskIndex]->PauseInTaskQueue();
				}

				ResourcesBlocked.AddSet(ClaimedResources);
			}
			else
			{
				bHasNulls = true;

				UE_VLOG(this, LogGameplayTasks, Warning, TEXT("UpdateTaskActivations found null entry in task queue at index:%d!"), TaskIndex);
			}
		}

		for (int32 Idx = 0; Idx < ActivationList.Num(); Idx++)
		{
			// check if task wasn't already finished as a result of activating previous elements of this list
			if (IsValid(ActivationList[Idx])
				&& ActivationList[Idx]->IsFinished() == false)
			{
				ActivationList[Idx]->ActivateInTaskQueue();
			}
		}
	}
	
	SetCurrentlyClaimedResources(ResourcesClaimed);

	// remove all null entries after processing activation changes
	if (bHasNulls)
	{
		TaskPriorityQueue.RemoveAll([](UGameplayTask* Task) { return Task == nullptr; });
	}
}

void UGameplayTasksComponent::SetCurrentlyClaimedResources(FGameplayResourceSet NewClaimedSet)
{
	if (CurrentlyClaimedResources != NewClaimedSet)
	{
		FGameplayResourceSet ReleasedResources = FGameplayResourceSet(CurrentlyClaimedResources).RemoveSet(NewClaimedSet);
		FGameplayResourceSet ClaimedResources = FGameplayResourceSet(NewClaimedSet).RemoveSet(CurrentlyClaimedResources);
		CurrentlyClaimedResources = NewClaimedSet;
		OnClaimedResourcesChange.Broadcast(ClaimedResources, ReleasedResources);
	}
}

//----------------------------------------------------------------------//
// debugging
//----------------------------------------------------------------------//
#if WITH_GAMEPLAYTASK_DEBUG
FString UGameplayTasksComponent::GetTickingTasksDescription() const
{
	FString TasksDescription;
	for (auto& Task : TickingTasks)
	{
		if (Task)
		{
			TasksDescription += FString::Printf(TEXT("\n%s %s"), *GetTaskStateName(Task->GetState()), *Task->GetDebugDescription());
		}
		else
		{
			TasksDescription += TEXT("\nNULL");
		}
	}
	return TasksDescription;
}

FString UGameplayTasksComponent::GetKnownTasksDescription() const
{
	FString TasksDescription;
	for (auto& Task : KnownTasks)
	{
		if (Task)
		{
			TasksDescription += FString::Printf(TEXT("\n%s %s"), *GetTaskStateName(Task->GetState()), *Task->GetDebugDescription());
		}
		else
		{
			TasksDescription += TEXT("\nNULL");
		}
	}
	return TasksDescription;
}

FString UGameplayTasksComponent::GetTasksPriorityQueueDescription() const
{
	FString TasksDescription;
	for (auto Task : TaskPriorityQueue)
	{
		if (Task != nullptr)
		{
			TasksDescription += FString::Printf(TEXT("\n%s %s"), *GetTaskStateName(Task->GetState()), *Task->GetDebugDescription());
		}
		else
		{
			TasksDescription += TEXT("\nNULL");
		}
	}
	return TasksDescription;
}

FString UGameplayTasksComponent::GetTaskStateName(EGameplayTaskState Value)
{
	static const UEnum* Enum = StaticEnum<EGameplayTaskState>();
	check(Enum);
	return Enum->GetNameStringByValue(int64(Value));
}
#endif // WITH_GAMEPLAYTASK_DEBUG

FConstGameplayTaskIterator UGameplayTasksComponent::GetTickingTaskIterator() const
{
	return TickingTasks.CreateConstIterator();
}

FConstGameplayTaskIterator UGameplayTasksComponent::GetKnownTaskIterator() const
{
	return KnownTasks.CreateConstIterator();
}

FConstGameplayTaskIterator UGameplayTasksComponent::GetPriorityQueueIterator() const
{
	return TaskPriorityQueue.CreateConstIterator();
}

FConstGameplayTaskIterator UGameplayTasksComponent::GetSimulatedTaskIterator() const
{
	return SimulatedTasks.CreateConstIterator();
}

#if ENABLE_VISUAL_LOG

void UGameplayTasksComponent::GrabDebugSnapshot(FVisualLogEntry* Snapshot) const
{
	DescribeSelfToVisLog(Snapshot);
}

void UGameplayTasksComponent::DescribeSelfToVisLog(FVisualLogEntry* Snapshot) const
{
	static const FString CategoryName = "GameplayTasks";
	static const FString PriorityQueueName = "Priority Queue";
	static const FString OtherTasksName = "Other tasks";

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	if (!IsThisNotNull(this, "UGameplayTasksComponent::DescribeSelfToVisLog") || !IsValidChecked(this))
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	{
		return;
	}

	FString NotInQueueDesc;
	for (auto& Task : KnownTasks)
	{
		if (Task)
		{
			if (!Task->RequiresPriorityOrResourceManagement())
			{
				NotInQueueDesc += FString::Printf(TEXT("\n%s %s %s %s"),
					*GetTaskStateName(Task->GetState()), *Task->GetDebugDescription(),
					Task->IsTickingTask() ? TEXT("[TICK]") : TEXT(""),
					Task->IsSimulatedTask() ? TEXT("[REP]") : TEXT(""));
			}
		}
		else
		{
			NotInQueueDesc += TEXT("\nNULL");
		}
	}

	for (const TObjectPtr<UGameplayTask>& Task : SimulatedTasks)
	{
		if (Task && !KnownTasks.Contains(Task))
		{
			NotInQueueDesc += FString::Printf(TEXT("\n%s %s %s %s"),
					*GetTaskStateName(Task->GetState()), *Task->GetDebugDescription(),
					Task->IsTickingTask() ? TEXT("[TICK]") : TEXT(""),
					Task->IsSimulatedTask() ? TEXT("[REP]") : TEXT(""));
		}
	}

	FVisualLogStatusCategory StatusCategory(CategoryName);
	StatusCategory.Add(OtherTasksName, NotInQueueDesc);
	StatusCategory.Add(PriorityQueueName, GetTasksPriorityQueueDescription());

	Snapshot->Status.Add(StatusCategory);
}
#endif // ENABLE_VISUAL_LOG

EGameplayTaskRunResult UGameplayTasksComponent::RunGameplayTask(IGameplayTaskOwnerInterface& TaskOwner, UGameplayTask& Task, uint8 Priority, FGameplayResourceSet AdditionalRequiredResources, FGameplayResourceSet AdditionalClaimedResources)
{
	const FText NoneText = FText::FromString(TEXT("None"));

	if (Task.GetState() == EGameplayTaskState::Paused || Task.GetState() == EGameplayTaskState::Active)
	{
		// return as success if already running for the same owner, failure otherwise 
		return Task.GetTaskOwner() == &TaskOwner
			? (Task.GetState() == EGameplayTaskState::Paused ? EGameplayTaskRunResult::Success_Paused : EGameplayTaskRunResult::Success_Active)
			: EGameplayTaskRunResult::Error;
	}

	// this is a valid situation if the task has been created via "Construct Object" mechanics
	if (Task.GetState() == EGameplayTaskState::Uninitialized)
	{
		Task.InitTask(TaskOwner, Priority);
	}

	Task.AddRequiredResourceSet(AdditionalRequiredResources);
	Task.AddClaimedResourceSet(AdditionalClaimedResources);
	Task.ReadyForActivation();

	switch (Task.GetState())
	{
	case EGameplayTaskState::AwaitingActivation:
	case EGameplayTaskState::Paused:
		return EGameplayTaskRunResult::Success_Paused;
	case EGameplayTaskState::Active:
		return EGameplayTaskRunResult::Success_Active;
	case EGameplayTaskState::Finished:
		return EGameplayTaskRunResult::Success_Active;
	}

	return EGameplayTaskRunResult::Error;
}

bool UGameplayTasksComponent::AddSimulatedTask(UGameplayTask* NewTask)
{
	if (NewTask == nullptr)
	{
		return false;
	}

	if (SimulatedTasks.Find(NewTask) == INDEX_NONE)
	{
		SimulatedTasks.Add(NewTask);
		SetSimulatedTasksNetDirty();

		if (IsUsingRegisteredSubObjectList() && IsReadyForReplication())
		{
			AddReplicatedSubObject(NewTask, COND_SkipOwner);
		}

		return true;
	}

	return false;
}

void UGameplayTasksComponent::RemoveSimulatedTask(UGameplayTask* NewTask)
{
	if (SimulatedTasks.RemoveSingle(NewTask) > 0)
	{
		SetSimulatedTasksNetDirty();

		if (IsUsingRegisteredSubObjectList())
		{
			RemoveReplicatedSubObject(NewTask);
		}
	}
}

void UGameplayTasksComponent::SetSimulatedTasks(const TArray<UGameplayTask*>& NewSimulatedTasks)
{
	if (IsUsingRegisteredSubObjectList() && IsReadyForReplication())
	{
		// Unregister all current tasks
		for (UGameplayTask* OldGameplayTask : SimulatedTasks)
		{
			if (OldGameplayTask)
			{
				RemoveReplicatedSubObject(OldGameplayTask);
			}
		}

		SimulatedTasks.Reset(NewSimulatedTasks.Num());

		// Register the new tasks
		for (UGameplayTask* NewGameplayTask : NewSimulatedTasks)
		{
			if (NewGameplayTask)
			{
				AddReplicatedSubObject(NewGameplayTask, COND_SkipOwner);
				SimulatedTasks.Add(NewGameplayTask);
			}
		}
	}
	else
	{
		SimulatedTasks = NewSimulatedTasks;
	}

	SetSimulatedTasksNetDirty();
}

void UGameplayTasksComponent::SetSimulatedTasksNetDirty()
{
	MARK_PROPERTY_DIRTY_FROM_NAME(UGameplayTasksComponent, SimulatedTasks, this);
}

//----------------------------------------------------------------------//
// BP API
//----------------------------------------------------------------------//
EGameplayTaskRunResult UGameplayTasksComponent::K2_RunGameplayTask(TScriptInterface<IGameplayTaskOwnerInterface> TaskOwner, UGameplayTask* Task, uint8 Priority, TArray<TSubclassOf<UGameplayTaskResource> > AdditionalRequiredResources, TArray<TSubclassOf<UGameplayTaskResource> > AdditionalClaimedResources)
{
	const FText NoneText = FText::FromString(TEXT("None"));

	if (TaskOwner.GetInterface() == nullptr)
	{
		FMessageLog("PIE").Error(FText::Format(
			LOCTEXT("RunGameplayTaskNullOwner", "Tried running a gameplay task {0} while owner is None!"),
			Task ? FText::FromName(Task->GetFName()) : NoneText));
		return EGameplayTaskRunResult::Error;
	}

	IGameplayTaskOwnerInterface& OwnerInstance = *TaskOwner;

	if (Task == nullptr)
	{
		FMessageLog("PIE").Error(FText::Format(
			LOCTEXT("RunNullGameplayTask", "Tried running a None task for {0}"),
			FText::FromString(Cast<UObject>(&OwnerInstance)->GetName())
			));
		return EGameplayTaskRunResult::Error;
	}

	if (Task->GetState() == EGameplayTaskState::Paused || Task->GetState() == EGameplayTaskState::Active)
	{
		FMessageLog("PIE").Warning(FText::Format(
			LOCTEXT("RunNullGameplayTask", "Tried running a None task for {0}"),
			FText::FromString(Cast<UObject>(&OwnerInstance)->GetName())
			));
		// return as success if already running for the same owner, failure otherwise 
		return Task->GetTaskOwner() == &OwnerInstance 
			? (Task->GetState() == EGameplayTaskState::Paused ? EGameplayTaskRunResult::Success_Paused : EGameplayTaskRunResult::Success_Active)
			: EGameplayTaskRunResult::Error;
	}

	// this is a valid situation if the task has been created via "Construct Object" mechanics
	if (Task->GetState() == EGameplayTaskState::Uninitialized)
	{
		Task->InitTask(OwnerInstance, Priority);
	}

	Task->AddRequiredResourceSet(AdditionalRequiredResources);
	Task->AddClaimedResourceSet(AdditionalClaimedResources);
	Task->ReadyForActivation();

	switch (Task->GetState())
	{
	case EGameplayTaskState::AwaitingActivation:
	case EGameplayTaskState::Paused:
		return EGameplayTaskRunResult::Success_Paused;
	case EGameplayTaskState::Active:
		return EGameplayTaskRunResult::Success_Active;
	case EGameplayTaskState::Finished:
		return EGameplayTaskRunResult::Success_Active;
	}

	return EGameplayTaskRunResult::Error;
}

//----------------------------------------------------------------------//
// FGameplayResourceSet
//----------------------------------------------------------------------//
FString FGameplayResourceSet::GetDebugDescription() const
{
	static const int32 FlagsCount = sizeof(FFlagContainer) * 8;
	FFlagContainer FlagsCopy = Flags;
	int32 FlagIndex = 0;

#if WITH_GAMEPLAYTASK_DEBUG
	static_assert(FlagsCount < TNumericLimits<uint8>::Max());
	FString Description;
	for (; FlagIndex < FlagsCount && FlagsCopy != 0; ++FlagIndex)
	{
		if (FlagsCopy & (1 << FlagIndex))
		{
			Description += UGameplayTaskResource::GetDebugDescription( IntCastChecked<uint8>(FlagIndex) );
			Description += TEXT(' ');
		}

		FlagsCopy &= ~(1 << FlagIndex);
	}
	return Description;
#else
	TCHAR Description[FlagsCount + 1];
	for (; FlagIndex < FlagsCount && FlagsCopy != 0; ++FlagIndex)
	{
		Description[FlagIndex] = (FlagsCopy & (1 << FlagIndex)) ? TCHAR('1') : TCHAR('0');
		FlagsCopy &= ~(1 << FlagIndex);
	}
	Description[FlagIndex] = TCHAR('\0');
	return FString(Description);
#endif // WITH_GAMEPLAYTASK_DEBUG
}

#undef LOCTEXT_NAMESPACE

