// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameplayTasksModule.h"
#include "Stats/Stats.h"
#include "GameplayTask.h"
#include "GameplayTasksPrivate.h"

//////////////////////////////////////////////////////////////////////////
// FCharacterAIModule

class FGameplayTasksModule : public IGameplayTasksModule
{
public:
	virtual void StartupModule() override
	{
		check(GConfig);		
	}

	virtual void ShutdownModule() override
	{
	}
};

//////////////////////////////////////////////////////////////////////////

IMPLEMENT_MODULE(FGameplayTasksModule, GameplayTasks);
DEFINE_LOG_CATEGORY(LogGameplayTasks);
DEFINE_STAT(STAT_TickGameplayTasks);
