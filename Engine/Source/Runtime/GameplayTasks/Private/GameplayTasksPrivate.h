// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Stats/Stats.h"

DECLARE_STATS_GROUP(TEXT("GameplayTasks"), STATGROUP_GameplayTasks, STATCAT_Advanced);
DECLARE_CYCLE_STAT_EXTERN(TEXT("TickGameplayTasks"), STAT_TickGameplayTasks, STATGROUP_GameplayTasks, );
