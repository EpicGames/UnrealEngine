// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tasks/GameplayTask_ClaimResource.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GameplayTask_ClaimResource)

UGameplayTask_ClaimResource::UGameplayTask_ClaimResource(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{

}

UGameplayTask_ClaimResource* UGameplayTask_ClaimResource::ClaimResource(TScriptInterface<IGameplayTaskOwnerInterface> InTaskOwner, TSubclassOf<UGameplayTaskResource> ResourceClass, const uint8 Priority, const FName TaskInstanceName)
{
	return InTaskOwner.GetInterface() ? ClaimResource(*InTaskOwner, ResourceClass, Priority, TaskInstanceName) : nullptr;
}

UGameplayTask_ClaimResource* UGameplayTask_ClaimResource::ClaimResources(TScriptInterface<IGameplayTaskOwnerInterface> InTaskOwner, TArray<TSubclassOf<UGameplayTaskResource> > ResourceClasses, const uint8 Priority, const FName TaskInstanceName)
{
	return InTaskOwner.GetInterface() ? ClaimResources(*InTaskOwner, ResourceClasses, Priority, TaskInstanceName) : nullptr;
}

UGameplayTask_ClaimResource* UGameplayTask_ClaimResource::ClaimResource(IGameplayTaskOwnerInterface& InTaskOwner, const TSubclassOf<UGameplayTaskResource> ResourceClass, const uint8 Priority, const FName TaskInstanceName)
{
	if (!ResourceClass)
	{
		return nullptr;
	}

	UGameplayTask_ClaimResource* MyTask = NewTaskUninitialized<UGameplayTask_ClaimResource>();
	if (MyTask)
	{
		MyTask->InitTask(InTaskOwner, Priority);
		MyTask->InstanceName = TaskInstanceName;
		MyTask->AddClaimedResource(ResourceClass);
	}

	return MyTask;
}

UGameplayTask_ClaimResource* UGameplayTask_ClaimResource::ClaimResources(IGameplayTaskOwnerInterface& InTaskOwner, const TArray<TSubclassOf<UGameplayTaskResource> >& ResourceClasses, const uint8 Priority, const FName TaskInstanceName)
{
	if (ResourceClasses.Num() == 0)
	{
		return nullptr;
	}

	UGameplayTask_ClaimResource* MyTask = NewTaskUninitialized<UGameplayTask_ClaimResource>();
	if (MyTask)
	{
		MyTask->InitTask(InTaskOwner, Priority);
		MyTask->InstanceName = TaskInstanceName;
		for (const TSubclassOf<UGameplayTaskResource>& ResourceClass : ResourceClasses)
		{
			MyTask->AddClaimedResource(ResourceClass);
		}
	}

	return MyTask;
}

