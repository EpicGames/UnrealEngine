// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tasks/GameplayTask_SpawnActor.h"
#include "EngineGlobals.h"
#include "Engine/Engine.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GameplayTask_SpawnActor)

UGameplayTask_SpawnActor* UGameplayTask_SpawnActor::SpawnActor(TScriptInterface<IGameplayTaskOwnerInterface> TaskOwner, FVector SpawnLocation, FRotator SpawnRotation, TSubclassOf<AActor> Class, bool bSpawnOnlyOnAuthority)
{
	if (TaskOwner.GetInterface() == nullptr)
	{
		return nullptr;
	}

	bool bCanSpawn = true;
	if (bSpawnOnlyOnAuthority == true)
	{
		AActor* TaskOwnerActor = TaskOwner->GetGameplayTaskOwner(nullptr);
		if (TaskOwnerActor)
		{
			bCanSpawn = (TaskOwnerActor->GetLocalRole() == ROLE_Authority);
		}
		else
		{
			// @todo add warning here
		}
	}

	UGameplayTask_SpawnActor* MyTask = nullptr;
	if (bCanSpawn)
	{
		MyTask = NewTask<UGameplayTask_SpawnActor>(TaskOwner);
		if (MyTask)
		{
			MyTask->CachedSpawnLocation = SpawnLocation;
			MyTask->CachedSpawnRotation = SpawnRotation;
			MyTask->ClassToSpawn = Class;
		}
	}
	return MyTask;
}

bool UGameplayTask_SpawnActor::BeginSpawningActor(UObject* WorldContextObject, AActor*& SpawnedActor)
{
	UWorld* const World = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull);
	if (World)
	{
		SpawnedActor = World->SpawnActorDeferred<AActor>(ClassToSpawn, FTransform(CachedSpawnRotation, CachedSpawnLocation), NULL, NULL, ESpawnActorCollisionHandlingMethod::AlwaysSpawn);
	}
	
	if (SpawnedActor == nullptr)
	{
		DidNotSpawn.Broadcast(nullptr);
		return false;
	}

	return true;
}

void UGameplayTask_SpawnActor::FinishSpawningActor(UObject* WorldContextObject, AActor* SpawnedActor)
{
	if (SpawnedActor)
	{
		const FTransform SpawnTransform(CachedSpawnRotation, CachedSpawnLocation);		
		SpawnedActor->FinishSpawning(SpawnTransform);
		Success.Broadcast(SpawnedActor);
	}

	EndTask();
}

