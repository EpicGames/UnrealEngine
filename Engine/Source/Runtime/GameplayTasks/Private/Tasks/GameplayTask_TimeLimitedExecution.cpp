// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tasks/GameplayTask_TimeLimitedExecution.h"
#include "Engine/EngineTypes.h"
#include "TimerManager.h"
#include "VisualLogger/VisualLogger.h"
#include "GameplayTasksComponent.h"
#include "Engine/World.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GameplayTask_TimeLimitedExecution)

UGameplayTask_TimeLimitedExecution::UGameplayTask_TimeLimitedExecution(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	Time = 0.f;
	TimeStarted = 0.f;

	bTimeExpired = false;
	bChildTaskFinished = false;
}

UGameplayTask_TimeLimitedExecution* UGameplayTask_TimeLimitedExecution::LimitExecutionTime(IGameplayTaskOwnerInterface& InTaskOwner, float Time, const uint8 Priority, const FName InInstanceName)
{
	if (Time <= 0.f)
	{
		return nullptr;
	}

	UGameplayTask_TimeLimitedExecution* MyTask = NewTaskUninitialized<UGameplayTask_TimeLimitedExecution>();
	if (MyTask)
	{
		MyTask->InstanceName = InInstanceName;
		MyTask->InitTask(InTaskOwner, Priority);
		MyTask->Time = Time;
	}
	return MyTask;
}

void UGameplayTask_TimeLimitedExecution::Activate()
{
	// bail when there's no child task
	if (ChildTask == nullptr)
	{
		EndTask();
		return;
	}

	UWorld* World = GetWorld();
	TimeStarted = World->GetTimeSeconds();

	// Use a dummy timer handle as we don't need to store it for later but we don't need to look for something to clear
	FTimerHandle TimerHandle;
	World->GetTimerManager().SetTimer(TimerHandle, this, &UGameplayTask_TimeLimitedExecution::OnTimer, (float)Time, false);
	UE_VLOG(GetGameplayTasksComponent(), LogGameplayTasks, Verbose, TEXT("%s> started timeout: %.2fs for task:%s"), *GetName(), Time, *ChildTask->GetName());

	if (!ChildTask->IsActive())
	{
		ChildTask->ReadyForActivation();
	}
}

void UGameplayTask_TimeLimitedExecution::OnGameplayTaskActivated(UGameplayTask& Task)
{
	if (!IsActive())
	{
		ReadyForActivation();
	}
}

void UGameplayTask_TimeLimitedExecution::OnGameplayTaskDeactivated(UGameplayTask& Task)
{
	Super::OnGameplayTaskDeactivated(Task);

	if (Task.IsFinished())
	{
		if (!bTimeExpired && !bChildTaskFinished)
		{
			OnFinished.Broadcast();
		}

		bChildTaskFinished = true;
		EndTask();
	}
}

void UGameplayTask_TimeLimitedExecution::OnTimer()
{
	if (!bTimeExpired && !bChildTaskFinished)
	{
		UE_VLOG(GetGameplayTasksComponent(), LogGameplayTasks, Verbose, TEXT("%s> time expired!"), *GetName());
		OnTimeExpired.Broadcast();
	}

	bTimeExpired = true;
	EndTask();
}

FString UGameplayTask_TimeLimitedExecution::GetDebugString() const
{
	const double TimeLeft = Time - GetWorld()->TimeSince(TimeStarted);
	return FString::Printf(TEXT("TimeLimit for %s. Time: %.2f. TimeLeft: %.2f"), *GetNameSafe(GetChildTask()), Time, TimeLeft);
}

