// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tasks/GameplayTask_WaitDelay.h"
#include "Engine/EngineTypes.h"
#include "TimerManager.h"
#include "Engine/World.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GameplayTask_WaitDelay)

UGameplayTask_WaitDelay::UGameplayTask_WaitDelay(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	Time = 0.f;
	TimeStarted = 0.f;
}

UGameplayTask_WaitDelay* UGameplayTask_WaitDelay::TaskWaitDelay(TScriptInterface<IGameplayTaskOwnerInterface> TaskOwner, float Time, const uint8 Priority)
{
	UGameplayTask_WaitDelay* MyTask = NewTaskUninitialized<UGameplayTask_WaitDelay>();
	if (MyTask && TaskOwner.GetInterface() != nullptr)
	{
		MyTask->InitTask(*TaskOwner, Priority);
		MyTask->Time = Time;
	}
	return MyTask;
}

UGameplayTask_WaitDelay* UGameplayTask_WaitDelay::TaskWaitDelay(IGameplayTaskOwnerInterface& InTaskOwner, float Time, const uint8 Priority)
{
	UGameplayTask_WaitDelay* MyTask = NewTaskUninitialized<UGameplayTask_WaitDelay>();
	if (MyTask)
	{
		MyTask->InitTask(InTaskOwner, Priority);
		MyTask->Time = Time;
	}
	return MyTask;
}

void UGameplayTask_WaitDelay::Activate()
{
	UWorld* World = GetWorld();
	TimeStarted = World->GetTimeSeconds();

	if (Time <= 0.0f)
	{
		World->GetTimerManager().SetTimerForNextTick(this, &UGameplayTask_WaitDelay::OnTimeFinish);
	}
	else
	{
		// Use a dummy timer handle as we don't need to store it for later but we don't need to look for something to clear
		FTimerHandle TimerHandle;
		World->GetTimerManager().SetTimer(TimerHandle, this, &UGameplayTask_WaitDelay::OnTimeFinish, (float)Time, false);
	}
}

void UGameplayTask_WaitDelay::OnTimeFinish()
{
	OnFinish.Broadcast();
	EndTask();
}

FString UGameplayTask_WaitDelay::GetDebugString() const
{
	double TimeLeft = Time - GetWorld()->TimeSince(TimeStarted);
	return FString::Printf(TEXT("WaitDelay. Time: %.2f. TimeLeft: %.2f"), Time, TimeLeft);
}

