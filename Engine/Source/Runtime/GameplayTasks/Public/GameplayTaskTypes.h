// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "VisualLogger/VisualLoggerDefines.h"
#include "UObject/Object.h"
#include "UObject/WeakObjectPtr.h"
#include "Templates/Casts.h"

#ifndef WITH_GAMEPLAYTASK_DEBUG
#define WITH_GAMEPLAYTASK_DEBUG (WITH_UNREAL_DEVELOPER_TOOLS || !(UE_BUILD_SHIPPING || UE_BUILD_TEST) || UE_DEBUG_RENDERING_ENABLED)
#endif // WITH_GAMEPLAYTASK_DEBUG

namespace FGameplayTasks
{
	inline constexpr uint8 DefaultPriority = 127;
	inline constexpr uint8 ScriptedPriority = 192;
}

